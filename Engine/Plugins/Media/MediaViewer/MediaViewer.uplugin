{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"Description": "Media viewer to display and compare media.",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"CanContainContent": true,
	"Modules": [
		{
			"Name": "MediaViewer",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "MediaStream",
			"Enabled": true
		},
		{
			"Name": "MediaPlayerEditor",
			"Enabled": true
		}
	]
}
