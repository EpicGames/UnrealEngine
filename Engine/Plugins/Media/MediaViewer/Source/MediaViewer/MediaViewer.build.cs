// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MediaViewer : ModuleRules
{
	public MediaViewer(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		bUseUnity = false;

		PublicDependencyModuleNames.AddRange(
			new[]
			{
				"Core",
				"CoreUObject"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new[]
			{
				"AppFramework",
				"ApplicationCore",
				"ContentBrowser",
				"ContentBrowserData",
				"EditorWidgets",
				"Engine",
				"ImageCore",
				"ImgMedia",
				"InputCore",
				"LevelEditor",
				"MediaAssets",
				"MediaCompositing",
				"MediaPlayerEditor",
				"MediaStream",
				"Projects",
				"PropertyEditor",
				"RenderCore",
				"RHI",
				"SlateCore",
				"Slate",
				"TimeManagement",
				"ToolWidgets",
				"WorkspaceMenuStructure",
				"UMG",
				"UnrealEd"
			}
		);

		// CQTest is a Developer module; only link it when developer/editor tools are being built
		// so that it is never pulled into shipping or game-client binaries.
		if (Target.bBuildEditor && Target.bBuildDeveloperTools)
		{
			PrivateDependencyModuleNames.Add("CQTest");
		}
	}
}
