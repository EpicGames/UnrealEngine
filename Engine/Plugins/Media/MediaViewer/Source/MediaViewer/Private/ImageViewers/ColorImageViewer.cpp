// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageViewers/ColorImageViewer.h"

#include "Brushes/SlateColorBrush.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/Optional.h"
#include "Misc/TVariant.h"

#define LOCTEXT_NAMESPACE "ColorImageViewer"

namespace UE::MediaViewer::Private
{
	constexpr int32 BlockSize = 100;

	FLinearColor MakeRandomColor()
	{
		// Guaranteed random (https://xkcd.com/221/)
		return FLinearColor::Red;
	}
}

namespace UE::MediaViewer::Private
{

const FLazyName FColorImageViewer::ItemTypeName = TEXT("Color");

TSharedRef<FSlateColorBrush> FColorImageViewer::ColorBrush = MakeShared<FSlateColorBrush>(FLinearColor::White);

bool FColorImageViewer::FFactory::SupportsItemType(FName InItemType) const
{
	return InItemType == FColorImageViewer::ItemTypeName;
}

TSharedPtr<FMediaViewerLibraryItem> FColorImageViewer::FFactory::CreateLibraryItem(const FMediaViewerLibraryItem& InSavedItem) const
{
	return MakeShared<FColorImageViewer::FItem>(FPrivateToken(), InSavedItem);
}

FLinearColor FColorImageViewer::FItem::LoadFromString(const FString& InString)
{
	FLinearColor Color = FLinearColor::Black;

	if (!InString.IsEmpty())
	{
		Color.InitFromString(InString);
	}

	return Color;
}

FColorImageViewer::FItem::FItem(const FText& InName, const FText& InToolTip, const FLinearColor& InColor)
	: FItem(FGuid::NewGuid(), InName, InToolTip, InColor)
{
}

FColorImageViewer::FItem::FItem(const FGuid& InId, const FText& InName, const FText& InToolTip, const FLinearColor& InColor)
	: FMediaViewerLibraryItem(InId, InName, InToolTip, /* Transient */ false, InColor.ToString())
{
}

FColorImageViewer::FItem::FItem(FPrivateToken&& InPrivateToken, const FMediaViewerLibraryItem& InItem)
	: FMediaViewerLibraryItem(InItem.GetId(), InItem.Name, InItem.ToolTip, /* Transient */ false, InItem.GetStringValue())
{
}

FName FColorImageViewer::FItem::GetItemType() const
{
	return FColorImageViewer::ItemTypeName;
}

FText FColorImageViewer::FItem::GetItemTypeDisplayName() const
{
	return LOCTEXT("Color", "Color");
}

TSharedPtr<FSlateBrush> FColorImageViewer::FItem::CreateThumbnail()
{
	if (StringValue.IsEmpty())
	{
		return nullptr;
	}

	TSharedRef<FSlateColorBrush> ThumbnailBrush = MakeShared<FSlateColorBrush>(LoadFromString(StringValue));

	return ThumbnailBrush;
}

TSharedPtr<FMediaImageViewer> FColorImageViewer::FItem::CreateImageViewer() const
{
	if (StringValue.IsEmpty())
	{
		return nullptr;
	}

	if (Id.IsValid())
	{
		return MakeShared<FColorImageViewer>(Id, LoadFromString(StringValue), Name);
	}
	
	return MakeShared<FColorImageViewer>(LoadFromString(StringValue), Name);
}

TSharedPtr<FMediaViewerLibraryItem> FColorImageViewer::FItem::Clone() const
{
	return MakeShared<FColorImageViewer::FItem>(FPrivateToken(), *this);
}

FColorImageViewer::FColorImageViewer()
	: FColorImageViewer(UE::MediaViewer::Private::MakeRandomColor(), LOCTEXT("Color", "Color"))
{
}

FColorImageViewer::FColorImageViewer(const FLinearColor& InColor, const FText& InDisplayName)
	: FColorImageViewer(FGuid::NewGuid(), InColor, InDisplayName)
{
}

FColorImageViewer::FColorImageViewer(const FGuid& InId, const FLinearColor& InColor, const FText& InDisplayName)
	: FMediaImageViewer({
		InId,
		{UE::MediaViewer::Private::BlockSize, UE::MediaViewer::Private::BlockSize},
		1,
		InDisplayName
	})
{
	Brush = ColorBrush;
	GetPaintSettings().Tint = InColor;
}

TSharedPtr<FMediaViewerLibraryItem> FColorImageViewer::CreateLibraryItem() const
{
	return MakeShared<FColorImageViewer::FItem>(ImageInfo.Id, ImageInfo.DisplayName, ImageInfo.DisplayName, GetPaintSettings().Tint);
}

TOptional<TVariant<FColor, FLinearColor>> FColorImageViewer::GetPixelColor(const FIntPoint& InPixelCoords, int32 InMipLevel) const
{
	using namespace UE::MediaViewer::Private;

	if (InPixelCoords.X >= 0 && InPixelCoords.Y >= 0 && InPixelCoords.X < BlockSize && InPixelCoords.Y < BlockSize)
	{
		TVariant<FColor, FLinearColor> ColorVariant;
		ColorVariant.Set<FLinearColor>(GetPaintSettings().Tint);
		return ColorVariant;
	}
	
	return {};
}

FString FColorImageViewer::GetReferencerName() const
{
	static const FString ReferencerName = TEXT("FColorImageViewer");
	return ReferencerName;
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
