// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImageViewer/IMediaImageViewerFactory.h"
#include "ImageViewer/MediaImageViewer.h"
#include "Library/MediaViewerLibraryItem.h"

#include "Math/Color.h"
#include "Templates/SharedPointer.h"

namespace UE::MediaViewer::Private
{

class FColorImageViewer : public FMediaImageViewer
{
public:
	struct FFactory : public IMediaImageViewerFactory
	{
		FFactory()
		{
			Priority = 10000;
		}

		virtual bool SupportsAsset(const FAssetData& InAssetData) const override { return false; }
		virtual TSharedPtr<FMediaImageViewer> CreateImageViewer(const FAssetData& InAssetData) const override { return nullptr; }
		virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(const FAssetData& InAssetData) const override { return nullptr; }

		virtual bool SupportsObject(TNotNull<UObject*> InObject) const override { return false; }
		virtual TSharedPtr<FMediaImageViewer> CreateImageViewer(TNotNull<UObject*> InObject) const override { return nullptr; }
		virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(TNotNull<UObject*> InObject) const override { return nullptr; }

		virtual bool SupportsItemType(FName InItemType) const override;
		virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(const FMediaViewerLibraryItem& InSavedItem) const override;
	};

	struct FItem : public FMediaViewerLibraryItem
	{
		static FLinearColor LoadFromString(const FString& InString);

		FItem(const FText& InName, const FText& InToolTip, const FLinearColor& InColor);

		FItem(const FGuid& InId, const FText& InName, const FText& InToolTip, const FLinearColor& InColor);

		FItem(FPrivateToken&& InPrivateToken, const FMediaViewerLibraryItem& InItem);

		virtual ~FItem() override = default;

		//~ Begin FMediaViewerLibraryItem
		virtual FName GetItemType() const override;
		virtual FText GetItemTypeDisplayName() const override;
		virtual TSharedPtr<FSlateBrush> CreateThumbnail() override;
		virtual TSharedPtr<FMediaImageViewer> CreateImageViewer() const override;
		virtual TSharedPtr<FMediaViewerLibraryItem> Clone() const override;
		//~ End FMediaViewerLibraryItem
	};

	static const FLazyName ItemTypeName;

	FColorImageViewer();
	FColorImageViewer(const FLinearColor& InColor, const FText& InDisplayName);
	FColorImageViewer(const FGuid& InId, const FLinearColor& InColor, const FText& InDisplayName);

	virtual ~FColorImageViewer() = default;

	//~ Begin FMediaImageViewer
	virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem() const override;
	virtual bool SupportsMip() const override { return false; }
	virtual TOptional<TVariant<FColor, FLinearColor>> GetPixelColor(const FIntPoint& InPixelCoords, int32 InMipLevel) const override;
	//~ End FMediaImageViewer

	//~ Begin FGCObject
	virtual FString GetReferencerName() const override;
	//~ End FGCObject

protected:
	static TSharedRef<FSlateColorBrush> ColorBrush;
};

} // UE::MediaViewer::Private
