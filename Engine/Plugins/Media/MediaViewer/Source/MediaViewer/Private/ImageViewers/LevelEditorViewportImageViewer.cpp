// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageViewers/LevelEditorViewportImageViewer.h"

#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "LevelEditor.h"
#include "MediaViewerUtils.h"
#include "Modules/ModuleManager.h"
#include "Slate/SceneViewport.h"
#include "SLevelViewport.h"
#include "Widgets/StrongSlateImageBrush.h"

#define LOCTEXT_NAMESPACE "LevelEditorViewportImageViewer"

namespace UE::MediaViewer::Private
{

namespace LevelEditorViewport
{

	const FGuid ViewportGuids[4] = {
		FGuid(0xFDD736C4, 0xFD7A98CB, 0x7C28F833, 0xD00B4BB3),
		FGuid(0xB7929682, 0x749CED27, 0x357D74D5, 0xC5E19053),
		FGuid(0xD82E7C42, 0xB038A8A1, 0x9C608700, 0x9C3B589C),
		FGuid(0xF0FE7988, 0xE5DE498B, 0x4AC6026B, 0x255B3BEC)
	};

	int32 GetTabIndex(const FString& InValue)
	{
		if (InValue.Len() > 1 && InValue[1] == TEXT('_')
			&& InValue[0] >= TEXT('0') && InValue[0] <= TEXT('3'))
		{
			return InValue[0] - TEXT('0');
		}

		return 0;
	}

	int32 GetViewportIndex(const FString& InValue)
	{
		const int32 LastIndex = InValue.Len() - 1;

		for (int32 Index = LastIndex; Index >= 0; --Index)
		{
			if (InValue[Index] >= TEXT('0') && InValue[Index] <= TEXT('9'))
			{
				continue;
			}

			// No numbers
			if (Index == LastIndex)
			{
				return 0;
			}

			// We've found a non-number
			return FCString::Atoi(*InValue.Mid(Index + 1));
		}

		// Whole string is a number... somehow!
		return FCString::Atoi(*InValue);
	}
}

/** 
 * Can take either a string value (Viewport0..3) or a config key and return the string value (Viewport0..3) 
 * Now also processed tab index (X_...)
 */
FString ConfigKeyToStringValue(const FString& InConfigKey)
{
	return InConfigKey;
}

/** Could be a config key or a direct value (Viewport0..3) */
FText GetViewportDisplayName(const FString& InViewportValue)
{
	using namespace LevelEditorViewport;

	const int32 TabIndex = GetTabIndex(InViewportValue);
	const int32 ViewportIndex = GetViewportIndex(InViewportValue);

	return FText::FromString(FString(TEXT("Tab")) + FString::FromInt(TabIndex + 1)
		+ TEXT(":")
		+ FString(TEXT("Viewport")) + FString::FromInt(ViewportIndex + 1));
}

FText GetViewportTooltip(const FString& InViewportValue)
{
	return FText::GetEmpty();
}

const FLazyName FLevelEditorViewportImageViewer::ItemTypeName = TEXT("LevelEditorViewport");

bool FLevelEditorViewportImageViewer::FFactory::SupportsItemType(FName InItemType) const
{
	return InItemType == ItemTypeName;
}

TSharedPtr<FMediaViewerLibraryItem> FLevelEditorViewportImageViewer::FFactory::CreateLibraryItem(const FMediaViewerLibraryItem& InSavedItem) const
{
	return MakeShared<FLevelEditorViewportImageViewer::FItem>(FPrivateToken(), InSavedItem);
}

FGuid FLevelEditorViewportImageViewer::FItem::GetIdForViewport(const FString& InConfigKey, bool bCreateIfInvalid)
{
	if (!InConfigKey.IsEmpty())
	{
		using namespace LevelEditorViewport;

		const int32 TabIndex = FMath::Clamp(GetTabIndex(InConfigKey), 0, 3);
		const int32 ViewportIndex = FMath::Clamp(GetViewportIndex(InConfigKey), 0, 3);

		if (TabIndex == 0)
		{
			return ViewportGuids[ViewportIndex];
		}

		FGuid ViewportGuid = ViewportGuids[ViewportIndex];
		ViewportGuid.D += TabIndex;

		return ViewportGuid;
	}

	if (bCreateIfInvalid)
	{
		return FGuid::NewGuid();
	}

	return FGuid();
}

TSharedPtr<FSceneViewport> FLevelEditorViewportImageViewer::FItem::GetViewportFromConfigKey(const FString& InConfigKey)
{
	return GetViewportFromStringValue(ConfigKeyToStringValue(InConfigKey));
}

TSharedPtr<FSceneViewport> FLevelEditorViewportImageViewer::FItem::GetViewportFromStringValue(const FString& InStringValue)
{
	if (InStringValue.IsEmpty())
	{
		return nullptr;
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::Get().GetModuleChecked<FLevelEditorModule>("LevelEditor");

	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();

	if (!LevelEditor.IsValid())
	{
		return nullptr;
	}

	TMap<int32, TArray<TSharedPtr<SLevelViewport>>> Viewports = LevelEditor->GetViewportsByTab();

	using namespace LevelEditorViewport;

	const int32 TabIndex = GetTabIndex(InStringValue);
	const int32 ViewportIndex = GetViewportIndex(InStringValue);

	if (!Viewports.Contains(TabIndex))
	{
		return nullptr;
	}

	if (!Viewports[TabIndex].IsValidIndex(ViewportIndex))
	{
		return nullptr;
	}

	return Viewports[TabIndex][ViewportIndex]->GetSharedActiveViewport();
}

FLevelEditorViewportImageViewer::FItem::FItem(const FString& InConfigKey)
	: FItem(GetIdForViewport(InConfigKey, /* Create id if invalid */ true), InConfigKey)
{
}

FLevelEditorViewportImageViewer::FItem::FItem(const FGuid& InId, const FString& InConfigKey)
	: FMediaViewerLibraryItem(InId, GetViewportDisplayName(InConfigKey), GetViewportTooltip(InConfigKey), /* Transient */ false, ConfigKeyToStringValue(InConfigKey))
{
}

FLevelEditorViewportImageViewer::FItem::FItem(FPrivateToken&& InPrivateToken, const FMediaViewerLibraryItem& InItem)
	: FMediaViewerLibraryItem(InItem.GetId(), InItem.Name, InItem.ToolTip, /* Transient */ false, InItem.GetStringValue())
{
}

FName FLevelEditorViewportImageViewer::FItem::GetItemType() const
{
	return FLevelEditorViewportImageViewer::ItemTypeName;
}

FSlateColor FLevelEditorViewportImageViewer::FItem::GetItemTypeColor() const
{
	return GetClassColor(UWorld::StaticClass());
}

FText FLevelEditorViewportImageViewer::FItem::GetItemTypeDisplayName() const
{
	return LOCTEXT("LevelEditorViewport", "Level Editor Viewport");
}

UTextureRenderTarget2D* FLevelEditorViewportImageViewer::FItem::CreateRenderTargetThumbnail()
{
	if (StringValue.IsEmpty())
	{
		return nullptr;
	}

	TSharedPtr<FSceneViewport> Viewport = GetViewportFromStringValue(StringValue);

	if (!Viewport.IsValid())
	{
		return nullptr;
	}

	const FIntPoint RenderTargetSize = {128, 128};
	UTextureRenderTarget2D* RenderTarget = FMediaViewerUtils::CreateRenderTarget(RenderTargetSize, /* Transparent */ false);

	RenderViewport(
		TNotNull<FViewport*>(Viewport.Get()),
		TNotNull<UTextureRenderTarget2D*>(RenderTarget),
		FRenderComplete(),
		/* Resize render target */ true
	);

	return RenderTarget;
}

TSharedPtr<FSlateBrush> FLevelEditorViewportImageViewer::FItem::CreateThumbnail()
{
	UTextureRenderTarget2D* RenderTarget = CreateRenderTargetThumbnail();

	if (!RenderTarget)
	{
		return nullptr;
	}

	TSharedRef<FSlateImageBrush> ThumbnailBrush = MakeShared<FStrongSlateImageBrush>(RenderTarget, FVector2D(RenderTarget->GetSurfaceWidth(), RenderTarget->GetSurfaceDepth()));

	return ThumbnailBrush;
}

TSharedPtr<FMediaImageViewer> FLevelEditorViewportImageViewer::FItem::CreateImageViewer() const
{
	if (StringValue.IsEmpty())
	{
		return nullptr;
	}

	if (Id.IsValid())
	{
		return MakeShared<FLevelEditorViewportImageViewer>(Id, StringValue);
	}

	return MakeShared<FLevelEditorViewportImageViewer>(StringValue);
}

TSharedPtr<FMediaViewerLibraryItem> FLevelEditorViewportImageViewer::FItem::Clone() const
{
	if (StringValue.IsEmpty())
	{
		return nullptr;
	}

	return MakeShared<FLevelEditorViewportImageViewer::FItem>(FPrivateToken(), *this);
}

FLevelEditorViewportImageViewer::FLevelEditorViewportImageViewer(const FString& InViewportValue)
	: FLevelEditorViewportImageViewer(FItem::GetIdForViewport(InViewportValue, /* Create id if invalid */ true), InViewportValue)
{
}

FLevelEditorViewportImageViewer::FLevelEditorViewportImageViewer(const FGuid& InId, const FString& InViewportValue)
	: FSceneViewportImageViewer(InId, FItem::GetViewportFromConfigKey(InViewportValue), GetViewportDisplayName(InViewportValue))
	, StringValue(ConfigKeyToStringValue(InViewportValue))
{
}

TSharedPtr<FMediaViewerLibraryItem> FLevelEditorViewportImageViewer::CreateLibraryItem() const
{
	return MakeShared<FItem>(ImageInfo.Id, StringValue);
}

void FLevelEditorViewportImageViewer::PaintImage(FMediaImageSlatePaintParams& InPaintParams, const FMediaImageSlatePaintGeometry& InPaintGeometry)
{
	if (ViewportSettings.bRealTime && !ViewportWeak.IsValid())
	{
		if (StringValue.IsEmpty())
		{
			return;
		}

		TSharedPtr<FSceneViewport> NewViewport = FItem::GetViewportFromStringValue(StringValue);

		if (!NewViewport.IsValid())
		{
			return;
		}

		ViewportWeak = NewViewport;
	}

	FSceneViewportImageViewer::PaintImage(InPaintParams, InPaintGeometry);
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
