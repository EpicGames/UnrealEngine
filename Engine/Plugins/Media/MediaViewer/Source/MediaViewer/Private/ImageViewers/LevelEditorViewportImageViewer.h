// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImageViewer/IMediaImageViewerFactory.h"
#include "ImageViewers/SceneViewportImageViewer.h"
#include "Library/MediaViewerLibraryItem.h"

namespace UE::MediaViewer::Private
{

class FLevelEditorViewportImageViewer : public FSceneViewportImageViewer
{
public:
	struct FFactory : public IMediaImageViewerFactory
	{
		FFactory()
		{
			Priority = 5000;
		}

		virtual bool SupportsAsset(const FAssetData& InAssetData) const override { return false; }
		virtual TSharedPtr<FMediaImageViewer> CreateImageViewer(const FAssetData& InAssetData) const override { return nullptr; }
		virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(const FAssetData& InAssetData) const override { return nullptr; }

		virtual bool SupportsObject(TNotNull<UObject*> InObject) const override { return false; }
		virtual TSharedPtr<FMediaImageViewer> CreateImageViewer(TNotNull<UObject*> InObject) const override { return nullptr; }
		virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(TNotNull<UObject*> InObject) const override { return nullptr; }

		virtual bool SupportsItemType(FName InItemType) const override;
		virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(const FMediaViewerLibraryItem& InSavedItem) const override;
	};

	struct FItem : public FMediaViewerLibraryItem
	{
		static FGuid GetIdForViewport(const FString& InConfigKey, bool bCreateIfInvalid);
		static TSharedPtr<FSceneViewport> GetViewportFromConfigKey(const FString& InConfigKey);
		static TSharedPtr<FSceneViewport> GetViewportFromStringValue(const FString& InStringValue);

		FItem(const FString& InConfigKey);

		FItem(const FGuid& InId, const FString& InConfigKey);

		FItem(FPrivateToken&& InPrivateToken, const FMediaViewerLibraryItem& InItem);

		virtual ~FItem() override = default;

		//~ Begin FMediaViewerLibraryItem
		virtual FName GetItemType() const override;
		virtual FText GetItemTypeDisplayName() const override;
		virtual FSlateColor GetItemTypeColor() const override;
		virtual TSharedPtr<FSlateBrush> CreateThumbnail() override;
		virtual TSharedPtr<FMediaImageViewer> CreateImageViewer() const override;
		virtual TSharedPtr<FMediaViewerLibraryItem> Clone() const override;
		//~ End FMediaViewerLibraryItem

	protected:
		UTextureRenderTarget2D* CreateRenderTargetThumbnail();
	};

	static const FLazyName ItemTypeName;

	/** InStringValue should be Viewport0 to Viewport3 */
	FLevelEditorViewportImageViewer(const FString& InStringValue);
	FLevelEditorViewportImageViewer(const FGuid& InId, const FString& InStringValue);

	//~ Begin FMediaImageViewer
	virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem() const override;
	//~ End FMediaImageViewer
	
protected:
	FString StringValue;

	//~ Begin FMediaImageViewer
	virtual void PaintImage(FMediaImageSlatePaintParams& InPaintParams, const FMediaImageSlatePaintGeometry& InPaintGeometry) override;
	//~ End FMediaImageViewer
};

} // UE::MediaViewer::Private
