// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageViewers/MaterialInterfaceImageViewer.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Brushes/SlateImageBrush.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Math/Vector2D.h"
#include "MediaViewerUtils.h"
#include "Misc/Optional.h"
#include "Misc/TVariant.h"
#include "Widgets/StrongSlateImageBrush.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialInterfaceImageViewer)

#define LOCTEXT_NAMESPACE "MaterialInterfaceImageViewer"

namespace UE::MediaViewer::Private
{

const FLazyName FMaterialInterfaceImageViewer::ItemTypeName = TEXT("MaterialInterface");

bool FMaterialInterfaceImageViewer::FFactory::SupportsAsset(const FAssetData& InAssetData) const
{
	if (InAssetData.AssetClassPath == UMaterial::StaticClass()->GetClassPathName())
	{
		return InAssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UMaterial, MaterialDomain)) == TEXT("MD_UI");
	}
	else if (InAssetData.AssetClassPath == UMaterialInstanceConstant::StaticClass()->GetClassPathName() || InAssetData.AssetClassPath == UMaterialInstanceDynamic::StaticClass()->GetClassPathName())
	{
		// Find the parent material
		FAssetRegistryModule& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		FAssetData CurrentData = InAssetData;
		int32 ParentCount = 0;
		constexpr int32 MaxParentCheck = 20;

		while (CurrentData.IsValid() && CurrentData.AssetClassPath != UMaterial::StaticClass()->GetClassPathName())
		{
			FName Parent = CurrentData.GetTagValueRef<FName>(GET_MEMBER_NAME_CHECKED(UMaterialInstance, Parent));
			CurrentData = AssetRegistry.Get().GetAssetByObjectPath(FSoftObjectPath(Parent.ToString()));

			++ParentCount;

			if (ParentCount >= MaxParentCheck)
			{
				UE_LOGF(LogMediaViewer, Error, "Unable to resolve material parent for \"%ls\".", *InAssetData.GetObjectPathString());
				break;
			}
		}

		if (CurrentData.IsValid())
		{
			return CurrentData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UMaterial, MaterialDomain)) == TEXT("MD_UI");
		}
	}

	return false;
}

TSharedPtr<FMediaImageViewer> FMaterialInterfaceImageViewer::FFactory::CreateImageViewer(const FAssetData& InAssetData) const
{
	if (UObject* Object = InAssetData.GetAsset())
	{
		return CreateImageViewer(TNotNull<UObject*>(Object));
	}

	return nullptr;
}

TSharedPtr<FMediaViewerLibraryItem> FMaterialInterfaceImageViewer::FFactory::CreateLibraryItem(const FAssetData& InAssetData) const
{
	if (UObject* Object = InAssetData.GetAsset())
	{
		return CreateLibraryItem(TNotNull<UObject*>(Object));
	}

	return nullptr;
}

bool FMaterialInterfaceImageViewer::FFactory::SupportsObject(TNotNull<UObject*> InObject) const
{
	if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(InObject))
	{
		return MaterialInterface->IsUIMaterial();
	}

	return false;
}

TSharedPtr<FMediaImageViewer> FMaterialInterfaceImageViewer::FFactory::CreateImageViewer(TNotNull<UObject*> InObject) const
{
	if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(InObject))
	{
		return MakeShared<FMaterialInterfaceImageViewer>(MaterialInterface);
	}

	return nullptr;
}

TSharedPtr<FMediaViewerLibraryItem> FMaterialInterfaceImageViewer::FFactory::CreateLibraryItem(TNotNull<UObject*> InObject) const
{
	if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(InObject))
	{
		return MakeShared<FMaterialInterfaceImageViewer::FItem>(
			FMediaImageViewer::GetObjectDisplayName(MaterialInterface),
			FText::Format(LOCTEXT("ToolTipFromat", "{0} [Material]"), FText::FromString(MaterialInterface->GetPathName())),
			MaterialInterface->HasAnyFlags(EObjectFlags::RF_Transient) || MaterialInterface->IsIn(GetTransientPackage()),
			MaterialInterface
		);
	}

	return nullptr;
}

bool FMaterialInterfaceImageViewer::FFactory::SupportsItemType(FName InItemType) const
{
	return InItemType == FMaterialInterfaceImageViewer::ItemTypeName;
}

TSharedPtr<FMediaViewerLibraryItem> FMaterialInterfaceImageViewer::FFactory::CreateLibraryItem(const FMediaViewerLibraryItem& InSavedItem) const
{
	FAssetData AssetData;

	if (IAssetRegistry::Get()->TryGetAssetByObjectPath(InSavedItem.GetStringValue(), AssetData) == UE::AssetRegistry::EExists::Exists)
	{
		return MakeShared<FMaterialInterfaceImageViewer::FItem>(FPrivateToken(), InSavedItem);
	}

	return nullptr;
}

FMaterialInterfaceImageViewer::FItem::FItem(const FText& InName, const FText& InToolTip, bool bInTransient, TNotNull<UMaterialInterface*> InMaterial)
	: FItem(FGuid::NewGuid(), InName, InToolTip, bInTransient, InMaterial)
{
}

FMaterialInterfaceImageViewer::FItem::FItem(const FGuid& InId, const FText& InName, const FText& InToolTip, bool bInTransient,
	TNotNull<UMaterialInterface*> InMaterial)
	: FMediaViewerLibraryItem(InId, InName, InToolTip, bInTransient, InMaterial->GetPathName())
{
}

FMaterialInterfaceImageViewer::FItem::FItem(FPrivateToken&& InPrivateToken, const FMediaViewerLibraryItem& InItem)
	: FMediaViewerLibraryItem(InItem.GetId(), InItem.Name, InItem.ToolTip, InItem.IsTransient(), InItem.GetStringValue())
{
}

FName FMaterialInterfaceImageViewer::FItem::GetItemType() const
{
	return FMaterialInterfaceImageViewer::ItemTypeName;
}

FSlateColor FMaterialInterfaceImageViewer::FItem::GetItemTypeColor() const
{
	return GetClassColor(UMaterial::StaticClass());
}

FText FMaterialInterfaceImageViewer::FItem::GetItemTypeDisplayName() const
{
	return LOCTEXT("Material", "Material");
}

UTextureRenderTarget2D* FMaterialInterfaceImageViewer::FItem::CreateRenderTargetThumbnail()
{
	if (StringValue.IsEmpty())
	{
		return nullptr;
	}

	UMaterialInterface* MaterialInterface = LoadAssetFromString<UMaterialInterface>(StringValue);

	if (!MaterialInterface)
	{
		return nullptr;
	}

	return FMediaViewerUtils::RenderMaterial(MaterialInterface);
}

TSharedPtr<FMediaViewerLibraryItem> FMaterialInterfaceImageViewer::FItem::Clone() const
{
	if (StringValue.IsEmpty())
	{
		return nullptr;
	}

	return MakeShared<FMaterialInterfaceImageViewer::FItem>(FPrivateToken(), *this);
}

TOptional<FAssetData> FMaterialInterfaceImageViewer::FItem::AsAsset() const
{
	if (UMaterialInterface* MaterialInterface = LoadAssetFromString<UMaterialInterface>(StringValue))
	{
		if (MaterialInterface->IsAsset())
		{
			return FAssetData(MaterialInterface);
		}
	}

	return {};
}

TSharedPtr<FSlateBrush> FMaterialInterfaceImageViewer::FItem::CreateThumbnail()
{
	if (StringValue.IsEmpty())
	{
		return nullptr;
	}

	UTextureRenderTarget2D* RenderTarget = CreateRenderTargetThumbnail();

	if (!RenderTarget)
	{
		return nullptr;
	}

	TSharedRef<FSlateImageBrush> ThumbnailBrush = MakeShared<FStrongSlateImageBrush>(RenderTarget, FVector2D(RenderTarget->GetSurfaceWidth(), RenderTarget->GetSurfaceDepth()));

	return ThumbnailBrush;
}

TSharedPtr<FMediaImageViewer> FMaterialInterfaceImageViewer::FItem::CreateImageViewer() const
{
	if (StringValue.IsEmpty())
	{
		return nullptr;
	}

	UMaterialInterface* MaterialInterface = LoadAssetFromString<UMaterialInterface>(StringValue);

	if (!MaterialInterface)
	{
		return nullptr;
	}

	if (Id.IsValid())
	{
		return MakeShared<FMaterialInterfaceImageViewer>(Id, MaterialInterface);
	}

	return MakeShared<FMaterialInterfaceImageViewer>(MaterialInterface);
}

FMaterialInterfaceImageViewer::FMaterialInterfaceImageViewer(TNotNull<UMaterialInterface*> InMaterialInterface)
	: FMaterialInterfaceImageViewer(FGuid::NewGuid(), InMaterialInterface)
{
}

FMaterialInterfaceImageViewer::FMaterialInterfaceImageViewer(const FGuid& InId, TNotNull<UMaterialInterface*> InMaterialInterface)
	: FMediaImageViewer({
		InId,
		FIntPoint(1, 1),
		1,
		FMediaImageViewer::GetObjectDisplayName(InMaterialInterface)
	})
{
	DrawEffects |= ESlateDrawEffect::PreMultipliedAlpha | ESlateDrawEffect::NoGamma;

	MaterialSettings.MaterialInterface = InMaterialInterface;

	CreateBrush();

	UMaterial::OnMaterialCompilationFinished().AddRaw(this, &FMaterialInterfaceImageViewer::OnMaterialCompiled);
}

FMaterialInterfaceImageViewer::~FMaterialInterfaceImageViewer()
{
	if (UObjectInitialized())
	{
		UMaterial::OnMaterialCompilationFinished().RemoveAll(this);
	}
}

TSharedPtr<FMediaViewerLibraryItem> FMaterialInterfaceImageViewer::CreateLibraryItem() const
{
	UMaterialInterface* MaterialInterface = MaterialSettings.MaterialInterface;

	if (!MaterialInterface)
	{
		return nullptr;
	}

	return MakeShared<FItem>(
		ImageInfo.Id,
		FMediaImageViewer::GetObjectDisplayName(MaterialInterface),
		FText::Format(LOCTEXT("ToolTipFromat", "{0} [Material]"), FText::FromString(MaterialInterface->GetPathName())),
		MaterialInterface->HasAnyFlags(EObjectFlags::RF_Transient) || MaterialInterface->IsIn(GetTransientPackage()),
		MaterialInterface
	);
}

TOptional<TVariant<FColor, FLinearColor>> FMaterialInterfaceImageViewer::GetPixelColor(const FIntPoint& InPixelCoords, int32 InMipLevel) const
{
	if (!SampleCache.IsValid() || !SampleCache->IsValid())
	{
		return {};
	}

	if (InPixelCoords.X < 0 || InPixelCoords.Y < 0)
	{
		SampleCache->Invalidate();
		return {};
	}

	if (InPixelCoords.X >= ImageInfo.Size.X || InPixelCoords.Y >= ImageInfo.Size.Y)
	{
		SampleCache->Invalidate();
		return {};
	}

	FIntPoint MirroredPixelCoords = InPixelCoords;

	if (EnumHasAllFlags(static_cast<EMediaImageMirrorPlane>(PaintSettings.MirrorPlanes), EMediaImageMirrorPlane::Horizontal))
	{
		MirroredPixelCoords.X = ImageInfo.Size.X - InPixelCoords.X - 1;
	}

	if (EnumHasAllFlags(static_cast<EMediaImageMirrorPlane>(PaintSettings.MirrorPlanes), EMediaImageMirrorPlane::Vertical))
	{
		MirroredPixelCoords.Y = ImageInfo.Size.Y - InPixelCoords.Y - 1;
	}

	if (const FLinearColor* PixelColor = SampleCache->GetPixelColor(MirroredPixelCoords))
	{
		TVariant<FColor, FLinearColor> PixelColorVariant;
		PixelColorVariant.Set<FLinearColor>(FMediaViewerUtils::CorrectGamma(*PixelColor));

		return PixelColorVariant;
	}

	return {};
}

TSharedPtr<FStructOnScope> FMaterialInterfaceImageViewer::GetCustomSettingsOnScope() const
{
	return MakeShared<FStructOnScope>(
		FMaterialInterfaceImageViewerSettings::StaticStruct(), 
		reinterpret_cast<uint8*>(&const_cast<FMaterialInterfaceImageViewerSettings&>(MaterialSettings))
	);
}

void FMaterialInterfaceImageViewer::PaintImage(FMediaImageSlatePaintParams& InPaintParams, const FMediaImageSlatePaintGeometry& InPaintGeometry)
{
	if (MaterialSettings.bRealTime)
	{
		RenderMaterial();
	}

	FMediaImageViewer::PaintImage(InPaintParams, InPaintGeometry);
}

void FMaterialInterfaceImageViewer::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged)
{
	FMediaImageViewer::NotifyPostChange(InPropertyChangedEvent, InPropertyThatChanged);

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(FMaterialInterfaceImageViewerSettings, RenderTargetSize))
	{
		CreateBrush();
	}
	else if (MemberName == GET_MEMBER_NAME_CHECKED(FMediaImagePaintSettings, MirrorPlanes))
	{
		OnMirrored();
	}
}

void FMaterialInterfaceImageViewer::OnMirrored()
{
	if (SampleCache.IsValid())
	{
		SampleCache->Invalidate();
	}
}

FString FMaterialInterfaceImageViewer::GetReferencerName() const
{
	static const FString ReferencerName = TEXT("FMaterialInterfaceImageViewer");
	return ReferencerName;
}

void FMaterialInterfaceImageViewer::AddReferencedObjects(FReferenceCollector& InCollector)
{
	FMediaImageViewer::AddReferencedObjects(InCollector);

	InCollector.AddPropertyReferencesWithStructARO(
		FMaterialInterfaceImageViewerSettings::StaticStruct(),
		&MaterialSettings
	);
}

void FMaterialInterfaceImageViewer::CreateBrush()
{
	ImageInfo.Size.X = MaterialSettings.RenderTargetSize;
	ImageInfo.Size.Y = MaterialSettings.RenderTargetSize;

	UTextureRenderTarget2D* RenderTarget = FMediaViewerUtils::RenderMaterial(MaterialSettings.MaterialInterface);
	MaterialSettings.RenderTarget = RenderTarget;

	Brush = MakeShared<FSlateImageBrush>(RenderTarget, FVector2D(ImageInfo.Size.X, ImageInfo.Size.Y));

	SampleCache = MakeShared<FTextureSampleCache>(RenderTarget, RenderTarget->GetFormat());
}

void FMaterialInterfaceImageViewer::RenderMaterial()
{
	if (::IsValid(MaterialSettings.MaterialInterface) && ::IsValid(MaterialSettings.RenderTarget))
	{
		bInvalidatedMipTexture = true;

		FMediaViewerUtils::RenderMaterial(MaterialSettings.MaterialInterface, MaterialSettings.RenderTarget);

		if (SampleCache.IsValid())
		{
			SampleCache->MarkDirty();
		}
	}
}

void FMaterialInterfaceImageViewer::OnMaterialCompiled(UMaterialInterface* InMaterialInterface)
{
	// We're already rendering every frame.
	if (MaterialSettings.bRealTime)
	{
		return;
	}

	UMaterialInterface* MyMaterial = MaterialSettings.MaterialInterface;

	if (!::IsValid(MyMaterial))
	{
		return;
	}

	bool bNeedsRender = false;
	
	if (InMaterialInterface == MyMaterial)
	{
		bNeedsRender = true;
	}

	if (UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(InMaterialInterface))
	{
		if (MaterialInstance->GetMaterial() == MyMaterial)
		{
			bNeedsRender = true;
		}
	}

	if (!bNeedsRender)
	{
		return;
	}

	if (::IsValid(MaterialSettings.RenderTarget))
	{
		RenderMaterial();
	}
	else
	{
		CreateBrush();
	}
}

bool FMaterialInterfaceImageViewer::OnOwnerCleanup(UObject* InObject)
{
	if (InObject && MaterialSettings.MaterialInterface && 
		(MaterialSettings.MaterialInterface == InObject || MaterialSettings.MaterialInterface->IsIn(InObject)))
	{
		SampleCache.Reset();

		if (Brush.IsValid())
		{
			Brush->SetResourceObject(nullptr);
		}

		return true;
	}

	return false;
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
