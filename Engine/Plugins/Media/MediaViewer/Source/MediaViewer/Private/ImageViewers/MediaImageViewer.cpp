// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageViewer/MediaImageViewer.h"

#include "Engine/Canvas.h"
#include "Engine/Texture.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector2D.h"
#include "MediaViewerUtils.h"
#include "Misc/TVariant.h"
#include "SlateMaterialBrush.h"
#include "Widgets/SMediaViewer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaImageViewer)

namespace UE::MediaViewer
{

const FSlateColorBrush FMediaImageViewer::BackgroundColorBrush(FLinearColor::White);

FText FMediaImageViewer::GetObjectDisplayName(const UObject* InObject)
{
	if (!InObject)
	{
		return FText::GetEmpty();
	}

	if (AActor* Actor = InObject->GetTypedOuter<AActor>())
	{
		const FString ActorLabel = Actor->GetActorLabel(/* Create if none */ false);

		if (ActorLabel.IsEmpty())
		{
			return FText::FromName(Actor->GetFName());
		}

		return FText::FromString(ActorLabel);
	}

	return FText::FromName(InObject->GetFName());
}

FMediaImageViewer::FMediaImageViewer(const FMediaImageViewerInfo& InImageInfo)
	: ImageInfo(InImageInfo)
	, BackgroundImageBrush(static_cast<UObject*>(nullptr), FVector2D(1))
	, Brush(nullptr)
{
}

void FMediaImageViewer::UpdateId(const FGuid& InId)
{
	ImageInfo.Id = InId;
}

void FMediaImageViewer::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged)
{
	InitBackgroundTexture();

	if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(FMediaImagePaintSettings, MipOverride)
		|| InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(FMediaImagePaintSettings, bAutoFilteringEnabled)
		|| InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(FMediaImagePaintSettings, AutoFilteringThreshold))
	{
		// This should already be invalidated by the mip level changing the required render texture size, but we'll make sure.
		bInvalidatedMipTexture = true;
	}
}

void FMediaImageViewer::AddReferencedObjects(FReferenceCollector& InCollector)
{
	if (Brush.IsValid())
	{
		if (UObject* Object = Brush->GetResourceObject())
		{
			InCollector.AddPropertyReferencesWithStructARO(
				FSlateBrush::StaticStruct(),
				Brush.Get()
			);
		}
	}
	
	if (UObject* Object = BackgroundImageBrush.GetResourceObject())
	{
		InCollector.AddPropertyReferencesWithStructARO(
			FSlateBrush::StaticStruct(),
			&BackgroundImageBrush
		);
	}
}

bool FMediaImageViewer::IsMirrored(EMediaImageMirrorPlane InPlane) const
{
	return EnumHasAllFlags(static_cast<EMediaImageMirrorPlane>(PaintSettings.MirrorPlanes), InPlane);
}

void FMediaImageViewer::SetMirrored(EMediaImageMirrorPlane InPlane, bool bInSet)
{
	if (IsMirrored(InPlane) == bInSet)
	{
		return;
	}

	if (bInSet)
	{
		PaintSettings.MirrorPlanes = PaintSettings.MirrorPlanes | static_cast<uint8>(InPlane);
	}
	else
	{
		PaintSettings.MirrorPlanes = PaintSettings.MirrorPlanes & ~static_cast<uint8>(InPlane);
	}

	OnMirrored();
}

FVector2D FMediaImageViewer::GetViewerCenter(const FVector2D& InViewerSize) const
{
	return InViewerSize * 0.5;
}

FVector2D FMediaImageViewer::GetPaintOffsetForViewerCenter(const FVector2D& InViewerSize) const
{
	return GetViewerCenter(InViewerSize) - (FVector2D(ImageInfo.Size.X, ImageInfo.Size.Y) * 0.5 * GetPaintSettings().Scale);
}

FVector2D FMediaImageViewer::GetPaintOffset(const FVector2D& InViewerSize, const FVector2D& InViewerPosition) const
{
	return GetPaintOffsetForViewerCenter(InViewerSize) + (FVector2D(GetPaintSettings().Offset)) + InViewerPosition;
}

FVector2D FMediaImageViewer::GetPaintSize() const
{
	return FVector2D(ImageInfo.Size.X, ImageInfo.Size.Y) * GetPaintSettings().Scale;
}

bool FMediaImageViewer::UpdateMipBrush()
{
	if (!Brush.IsValid())
	{
		return false;
	}

	UTexture* BrushTexture = Cast<UTexture>(Brush->GetResourceObject());

	if (!BrushTexture)
	{
		return false;
	}

	if (!MipAdjustedMaterial.IsValid())
	{
		UMaterial* MipAdjustedMaterialBase = LoadObject<UMaterial>(nullptr, Private::FMediaViewerUtils::MipRenderMaterialPath);
		MipAdjustedMaterial.Reset(UMaterialInstanceDynamic::Create(MipAdjustedMaterialBase, GetTransientPackage()));

		if (!MipAdjustedMaterial.IsValid())
		{
			return false;
		}

		bInvalidatedMipTexture = true;
	}

	// Create a render target that is the same size as the mip level.
	const FVector2D RequiredSizeDouble = 
		FVector2D(ImageInfo.Size.X, ImageInfo.Size.Y)
		/ FMath::Max(
			1.0, 
			static_cast<float>(FMath::Pow(2.0, PaintSettings.GetMipLevel())
		)
	);

	const FIntPoint RequiredSize = {
		static_cast<int32>(FMath::CeilToInt(RequiredSizeDouble.X)),
		static_cast<int32>(FMath::CeilToInt(RequiredSizeDouble.Y))
	};
	
	if (!MipAdjustedRenderTarget.IsValid())
	{
		MipAdjustedRenderTarget.Reset(Private::FMediaViewerUtils::CreateRenderTarget(RequiredSize, /* Transparent */ true));

		if (!MipAdjustedRenderTarget.IsValid())
		{
			return false;
		}

		MipAdjustedRenderTarget->Filter = TextureFilter::TF_Nearest;
		bInvalidatedMipTexture = true;
	}
	// If it was just created, we don't need to change the size.
	else if (MipAdjustedRenderTarget->SizeX != RequiredSize.X || MipAdjustedRenderTarget->SizeY != RequiredSize.Y)
	{
		MipAdjustedRenderTarget->ResizeTarget(RequiredSize.X, RequiredSize.Y);
		bInvalidatedMipTexture = true;
	}

	if (!MipAdjustedBrush.IsValid())
	{
		MipAdjustedBrush = MakeShared<FSlateImageBrush>(MipAdjustedRenderTarget.Get(), RequiredSizeDouble);
	}
	// If it was just created, we don't need to change the size.
	else if (!FMath::IsNearlyEqual(MipAdjustedBrush->ImageSize.X, RequiredSize.X)
		|| !FMath::IsNearlyEqual(MipAdjustedBrush->ImageSize.Y, RequiredSize.Y))
	{
		MipAdjustedBrush->SetImageSize(RequiredSizeDouble);
	}

	MipAdjustedMaterial->SetTextureParameterValue(TEXT("Texture"), BrushTexture);
	MipAdjustedMaterial->SetScalarParameterValue(TEXT("MipLevel"), static_cast<float>(PaintSettings.GetMipLevel()));

	if (bInvalidatedMipTexture)
	{
		// Fail the render rather than blocking on shader compile, the render will retry on next paint.
		constexpr Private::EShaderCompileFlags ShaderCompileFlags = Private::EShaderCompileFlags::None;
		if (!Private::FMediaViewerUtils::RenderMaterial(MipAdjustedMaterial.Get(), MipAdjustedRenderTarget.Get(), ShaderCompileFlags))
		{
			return false;
		}

		bInvalidatedMipTexture = false;
	}

	return true;
}

FSlateClippingZone FMediaImageViewer::CreateSlateClippingZone(const FSlateRect& InCullingRect, float InDPIScale,
	const FVector2D& InViewerPosition, EOrientation InOrientation, const FFloatRange& InUVRange) const
{
	constexpr float WindowBorderPadding = 4.f;

	const float Left = InCullingRect.Left + InViewerPosition.X * InDPIScale + WindowBorderPadding;
	const float Right = InCullingRect.Right - WindowBorderPadding;
	const float Top = InCullingRect.Top + InViewerPosition.Y * InDPIScale + WindowBorderPadding;
	const float Bottom = InCullingRect.Bottom - WindowBorderPadding;

	FSlateClippingZone ClippingZone;

	switch (InOrientation)
	{
		case Orient_Horizontal:
			ClippingZone.TopLeft.Y = Top;
			ClippingZone.TopRight.Y = Top;
			ClippingZone.BottomLeft.Y = Bottom;
			ClippingZone.BottomRight.Y = Bottom;

			ClippingZone.TopLeft.X = FMath::Lerp(Left, Right, InUVRange.GetLowerBound().GetValue());
			ClippingZone.TopRight.X = FMath::Lerp(Left, Right, InUVRange.GetUpperBound().GetValue());
			ClippingZone.BottomLeft.X = ClippingZone.TopLeft.X;
			ClippingZone.BottomRight.X = ClippingZone.TopRight.X;
			break;

		case Orient_Vertical:
			ClippingZone.TopLeft.Y = FMath::Lerp(Top, Bottom, InUVRange.GetLowerBound().GetValue());
			ClippingZone.BottomLeft.Y = FMath::Lerp(Top, Bottom, InUVRange.GetUpperBound().GetValue());
			ClippingZone.TopRight.Y = ClippingZone.TopLeft.Y;
			ClippingZone.BottomRight.Y = ClippingZone.BottomLeft.Y;

			ClippingZone.TopLeft.X = Left;
			ClippingZone.BottomLeft.X = Left;
			ClippingZone.TopRight.X = Right;
			ClippingZone.BottomRight.X = Right;
			break;
	}

	return ClippingZone;
}

void FMediaImageViewer::InitBackgroundTexture()
{
	if (UTexture* Texture = PanelSettings.BackgroundTexture.LoadSynchronous())
	{
		BackgroundImageBrush.SetResourceObject(Texture);
		BackgroundImageBrush.ImageSize.X = Texture->GetSurfaceWidth();
		BackgroundImageBrush.ImageSize.Y = Texture->GetSurfaceHeight();
	}
	else
	{
		BackgroundImageBrush.SetResourceObject(nullptr);
	}
}

void FMediaImageViewer::Paint(FMediaImageSlatePaintParams& InPaintParams)
{
	FVector2D PaintOffset = GetPaintOffset(InPaintParams.ViewerSize, InPaintParams.ViewerPosition);
	FVector2D PaintSize = GetPaintSize();
	FPaintGeometry PaintGeometry = InPaintParams.AllottedGeometry.ToPaintGeometry(PaintSize, FSlateLayoutTransform(PaintOffset));

	const FMediaImageSlatePaintGeometry Geometry = {
		MoveTemp(PaintOffset),
		MoveTemp(PaintSize),
		MoveTemp(PaintGeometry)
	};

	const FSlateClippingZone ClippingZone = CreateSlateClippingZone(
		InPaintParams.MyCullingRect,
		InPaintParams.DPIScale,
		InPaintParams.ViewerPosition,
		InPaintParams.Orientation,
		InPaintParams.UVRange
	);

	InPaintParams.DrawElements.PushClip(ClippingZone);

	PaintPanel(InPaintParams, Geometry);
	PaintImage(InPaintParams, Geometry);
	PaintPixelGrid(InPaintParams, Geometry);

	InPaintParams.DrawElements.PopClip();
}

void FMediaImageViewer::PaintPanel(FMediaImageSlatePaintParams& InPaintParams, const FMediaImageSlatePaintGeometry& InPaintGeometry)
{
	const FLinearColor AlphaTint = FLinearColor(1.f, 1.f, 1.f, InPaintParams.ImageOpacity);

	if (PanelSettings.BackgroundColor.IsSet())
	{
		FSlateDrawElement::MakeBox(
			InPaintParams.DrawElements,
			InPaintParams.LayerId,
			InPaintGeometry.Geometry,
			&BackgroundColorBrush,
			ESlateDrawEffect::NoPixelSnapping,
			PanelSettings.BackgroundColor.GetValue() * AlphaTint
		);

		++InPaintParams.LayerId;
	}

	if (BackgroundImageBrush.GetResourceObject())
	{
		FSlateDrawElement::MakeBox(
			InPaintParams.DrawElements,
			InPaintParams.LayerId,
			InPaintGeometry.Geometry,
			&BackgroundImageBrush,
			ESlateDrawEffect::NoPixelSnapping,
			AlphaTint
		);

		++InPaintParams.LayerId;
	}
}

void FMediaImageViewer::PaintImage(FMediaImageSlatePaintParams& InPaintParams, const FMediaImageSlatePaintGeometry& InPaintGeometry)
{
	TSharedPtr<FSlateBrush> BrushToUse = Brush;
	ESlateDrawEffect DrawEffectsToUse = DrawEffects;

	const bool bShouldDisableFiltering = PaintSettings.bAutoFilteringEnabled
		&& (PaintSettings.Scale >= PaintSettings.AutoFilteringThreshold);

	if (SupportsMip() && (PaintSettings.bEnableMipOverride || bShouldDisableFiltering))
	{
		if (UpdateMipBrush())
		{
			BrushToUse = MipAdjustedBrush;
			DrawEffectsToUse = ESlateDrawEffect::NoPixelSnapping | ESlateDrawEffect::NoGamma;
		}
	}

	if (!BrushToUse.IsValid() || !BrushToUse->GetResourceObject())
	{
		return;
	}

	switch (PaintSettings.MirrorPlanes & static_cast<uint8>(EMediaImageMirrorPlane::TwoDim))
	{
		case static_cast<uint8>(EMediaImageMirrorPlane::None):
			BrushToUse->Mirroring = ESlateBrushMirrorType::NoMirror;
			break;

		case static_cast<uint8>(EMediaImageMirrorPlane::Horizontal):
			BrushToUse->Mirroring = ESlateBrushMirrorType::Horizontal;
			break;

		case static_cast<uint8>(EMediaImageMirrorPlane::Vertical):
			BrushToUse->Mirroring = ESlateBrushMirrorType::Vertical;
			break;

		case static_cast<uint8>(EMediaImageMirrorPlane::Horizontal | EMediaImageMirrorPlane::Vertical):
			BrushToUse->Mirroring = ESlateBrushMirrorType::Both;
			break;
	}

	FSlateDrawElement::MakeRotatedBox(
		InPaintParams.DrawElements,
		InPaintParams.LayerId,
		InPaintGeometry.Geometry,
		BrushToUse.Get(),
		DrawEffectsToUse,
		GetPaintSettings().Rotation.Yaw * 2.f * PI / 360.f,
		{},
		FSlateDrawElement::ERotationSpace::RelativeToElement,
		GetPaintSettings().Tint * FLinearColor(1.f, 1.f, 1.f, InPaintParams.ImageOpacity)
	);

	++InPaintParams.LayerId;
}

void FMediaImageViewer::PaintPixelGrid(FMediaImageSlatePaintParams& InPaintParams, const FMediaImageSlatePaintGeometry& InPaintGeometry)
{
	// At mip level N each displayed "pixel" covers 2^N source pixels, so widen the
	// grid cell size and shrink the iteration bounds to match the mip resolution.
	const int32 MipLevel  = static_cast<int32>(PaintSettings.GetMipLevel());
	const float CellSize  = PaintSettings.Scale * static_cast<float>(1 << MipLevel);

	// Show the grid once a single source pixel occupies at least the configured screen-pixel threshold.
	const bool bShouldShowGrid = PaintSettings.bPixelGridEnabled
		&& PaintSettings.bAutoFilteringEnabled
		&& (CellSize >= PaintSettings.PixelGridThreshold);

	if (!bShouldShowGrid)
	{
		return;
	}
	const int32 MipWidth  = FMath::Max(1, ImageInfo.Size.X >> MipLevel);
	const int32 MipHeight = FMath::Max(1, ImageInfo.Size.Y >> MipLevel);

	const FVector2D& Position = InPaintGeometry.Position;

	// Intersect image bounds with viewer bounds to avoid drawing outside either
	const float ClipLeft   = FMath::Max(Position.X, InPaintParams.ViewerPosition.X);
	const float ClipTop    = FMath::Max(Position.Y, InPaintParams.ViewerPosition.Y);
	const float ClipRight  = FMath::Min(Position.X + InPaintGeometry.Size.X, InPaintParams.ViewerPosition.X + InPaintParams.ViewerSize.X);
	const float ClipBottom = FMath::Min(Position.Y + InPaintGeometry.Size.Y, InPaintParams.ViewerPosition.Y + InPaintParams.ViewerSize.Y);

	if (ClipLeft >= ClipRight || ClipTop >= ClipBottom)
	{
		return;
	}

	// Determine which pixel boundary lines are visible
	const int32 FirstCol = FMath::Max(0, FMath::FloorToInt((ClipLeft  - Position.X) / CellSize));
	const int32 LastCol  = FMath::Min(MipWidth,  FMath::CeilToInt((ClipRight  - Position.X) / CellSize));
	const int32 FirstRow = FMath::Max(0, FMath::FloorToInt((ClipTop   - Position.Y) / CellSize));
	const int32 LastRow  = FMath::Min(MipHeight, FMath::CeilToInt((ClipBottom - Position.Y) / CellSize));

	const FLinearColor GridColor(0.f, 0.f, 0.f, 0.4f);
	const FPaintGeometry FullGeometry = InPaintParams.AllottedGeometry.ToPaintGeometry();
	constexpr ESlateDrawEffect DrawEffect = ESlateDrawEffect::NoPixelSnapping;

	TArray<FVector2D> LinePoints;
	LinePoints.SetNumUninitialized(2);

	// Vertical lines (pixel column boundaries)
	for (int32 Col = FirstCol; Col <= LastCol; ++Col)
	{
		const float X = Position.X + Col * CellSize;
		LinePoints[0] = FVector2D(X, ClipTop);
		LinePoints[1] = FVector2D(X, ClipBottom);
		FSlateDrawElement::MakeLines(InPaintParams.DrawElements, InPaintParams.LayerId, FullGeometry, LinePoints, DrawEffect, GridColor, false, 1.f);
	}

	// Horizontal lines (pixel row boundaries)
	for (int32 Row = FirstRow; Row <= LastRow; ++Row)
	{
		const float Y = Position.Y + Row * CellSize;
		LinePoints[0] = FVector2D(ClipLeft, Y);
		LinePoints[1] = FVector2D(ClipRight, Y);
		FSlateDrawElement::MakeLines(InPaintParams.DrawElements, InPaintParams.LayerId, FullGeometry, LinePoints, DrawEffect, GridColor, false, 1.f);
	}

	++InPaintParams.LayerId;
}

void FMediaImageViewer::Paint(const FMediaImageCanvasPaintParams& InPaintParams)
{
	const FVector2D CanvasSize = {
		static_cast<double>(InPaintParams.Canvas->SizeX),
		static_cast<double>(InPaintParams.Canvas->SizeY)
	};

	const FMediaImageCanvasPaintGeometry PaintGeometry = {
		GetPaintOffset(CanvasSize, FVector2D::ZeroVector),
		GetPaintSize()
	};

	PaintPanel(InPaintParams, PaintGeometry);
	PaintImage(InPaintParams, PaintGeometry);
}

void FMediaImageViewer::PaintPanel(const FMediaImageCanvasPaintParams& InPaintParams, const FMediaImageCanvasPaintGeometry& InPaintGeometry)
{
	const FLinearColor AlphaTint = FLinearColor(1.f, 1.f, 1.f, InPaintParams.ImageOpacity);

	if (PanelSettings.BackgroundColor.IsSet())
	{
		static TSoftObjectPtr<UTexture> WhiteTexturePtr = TSoftObjectPtr<UTexture>(FSoftObjectPath(TEXT("/Script/Engine.Texture2D'/Engine/EngineResources/WhiteSquareTexture.WhiteSquareTexture'")));
		UTexture* WhiteTexture = WhiteTexturePtr.LoadSynchronous();

		InPaintParams.Canvas->SetLinearDrawColor(PanelSettings.BackgroundColor.GetValue() * AlphaTint);
		
		InPaintParams.Canvas->DrawTile(
			WhiteTexture,
			InPaintGeometry.Position.X,
			InPaintGeometry.Position.Y,
			InPaintGeometry.Size.X,
			InPaintGeometry.Size.Y,
			0,
			0,
			WhiteTexture->GetSurfaceWidth(),
			WhiteTexture->GetSurfaceHeight()
		);

		InPaintParams.Canvas->SetLinearDrawColor(FLinearColor::White);
	}

	if (UTexture* BackgroundTexture = Cast<UTexture>(BackgroundImageBrush.GetResourceObject()))
	{
		InPaintParams.Canvas->DrawTile(
			BackgroundTexture,
			InPaintGeometry.Position.X,
			InPaintGeometry.Position.Y,
			InPaintGeometry.Size.X,
			InPaintGeometry.Size.Y,
			0,
			0,
			BackgroundTexture->GetSurfaceWidth(),
			BackgroundTexture->GetSurfaceHeight()
		);
	}
}

void FMediaImageViewer::PaintImage(const FMediaImageCanvasPaintParams& InPaintParams, const FMediaImageCanvasPaintGeometry& InPaintGeometry)
{
	TSharedPtr<FSlateBrush> BrushToUse = Brush;

	if (SupportsMip() && PaintSettings.bEnableMipOverride)
	{
		if (UpdateMipBrush())
		{
			BrushToUse = MipAdjustedBrush;
		}
	}

	if (!BrushToUse.IsValid())
	{
		return;
	}

	const FVector2D CanvasUVRange = FVector2D(0, 1);

	switch (InPaintParams.Orientation)
	{
		case EOrientation::Orient_Horizontal:
		{
			const FVector2D ImageCanvasUVRange = FVector2D(
				-InPaintGeometry.Position.X / InPaintGeometry.Size.X,
				(InPaintParams.Canvas->SizeX - InPaintGeometry.Position.X) / InPaintGeometry.Size.X
			);

			// UV Range is based on the canvas, not the image. Convert to image range and clamp at 0-1.
			const float ImageUVRangeMinX = FMath::Clamp(FMath::GetMappedRangeValueUnclamped(
				CanvasUVRange,
				ImageCanvasUVRange,
				InPaintParams.UVRange.GetLowerBound().GetValue()
			), 0, 1);

			const float ImageUVRangeMaxX = FMath::Clamp(FMath::GetMappedRangeValueUnclamped(
				CanvasUVRange,
				ImageCanvasUVRange,
				InPaintParams.UVRange.GetUpperBound().GetValue()
			), 0, 1);

			const float ImageUVRangeSize = ImageUVRangeMaxX - ImageUVRangeMinX;

			const float PositionX = InPaintGeometry.Position.X + (ImageUVRangeMinX * InPaintGeometry.Size.X);
			const float SizeX = InPaintGeometry.Size.X * ImageUVRangeSize;

			if (UTexture* ForegroundTexture = Cast<UTexture>(BrushToUse->GetResourceObject()))
			{
				InPaintParams.Canvas->K2_DrawTexture(
					ForegroundTexture,
					{ PositionX, InPaintGeometry.Position.Y },
					{ SizeX, InPaintGeometry.Size.Y },
					FVector2D(ImageUVRangeMinX, 0),
					FVector2D(ImageUVRangeSize, 1),
					GetPaintSettings().Tint * FLinearColor(1.f, 1.f, 1.f, InPaintParams.ImageOpacity),
					BLEND_Translucent,
					GetPaintSettings().Rotation.Yaw
				);
			}
			else if (UMaterialInterface* ForegroundMaterial = Cast<UMaterialInterface>(BrushToUse->GetResourceObject()))
			{
				InPaintParams.Canvas->K2_DrawMaterial(
					ForegroundMaterial,
					{ PositionX, InPaintGeometry.Position.Y },
					{ SizeX, InPaintGeometry.Size.Y },
					FVector2D(ImageUVRangeMinX, 0),
					FVector2D(ImageUVRangeSize, 1),
					GetPaintSettings().Rotation.Yaw
				);
			}

			break;
		}

		case EOrientation::Orient_Vertical:
		{
			const FVector2D ImageCanvasUVRange = FVector2D(
				-InPaintGeometry.Position.Y / InPaintGeometry.Size.Y,
				(InPaintParams.Canvas->SizeY - InPaintGeometry.Position.Y) / InPaintGeometry.Size.Y
			);

			// UV Range is based on the canvas, not the image. Convert to image range and clamp at 0-1.
			const float ImageUVRangeMinY = FMath::Clamp(FMath::GetMappedRangeValueUnclamped(
				CanvasUVRange,
				ImageCanvasUVRange,
				InPaintParams.UVRange.GetLowerBound().GetValue()
			), 0, 1);

			const float ImageUVRangeMaxY = FMath::Clamp(FMath::GetMappedRangeValueUnclamped(
				CanvasUVRange,
				ImageCanvasUVRange,
				InPaintParams.UVRange.GetUpperBound().GetValue()
			), 0, 1);

			const float ImageUVRangeSize = ImageUVRangeMaxY - ImageUVRangeMinY;

			const float PositionY = InPaintGeometry.Position.Y + (ImageUVRangeMinY * InPaintGeometry.Size.Y);
			const float SizeY = InPaintGeometry.Size.Y * ImageUVRangeSize;

			if (UTexture* ForegroundTexture = Cast<UTexture>(BrushToUse->GetResourceObject()))
			{
				InPaintParams.Canvas->K2_DrawTexture(
					ForegroundTexture,
					{ InPaintGeometry.Position.X, PositionY },
					{ InPaintGeometry.Size.X, SizeY },
					FVector2D(0, ImageUVRangeMinY),
					FVector2D(1, ImageUVRangeSize),
					GetPaintSettings().Tint * FLinearColor(1.f, 1.f, 1.f, InPaintParams.ImageOpacity),
					BLEND_Translucent,
					GetPaintSettings().Rotation.Yaw
				);
			}
			else if (UMaterialInterface* ForegroundMaterial = Cast<UMaterialInterface>(BrushToUse->GetResourceObject()))
			{
				InPaintParams.Canvas->K2_DrawMaterial(
					ForegroundMaterial,
					{ InPaintGeometry.Position.X, PositionY },
					{ InPaintGeometry.Size.X, SizeY },
					FVector2D(0, ImageUVRangeMinY),
					FVector2D(1, ImageUVRangeSize),
					GetPaintSettings().Rotation.Yaw
				);
			}

			break;
		}
	}
}

} // UE::MediaViewer
