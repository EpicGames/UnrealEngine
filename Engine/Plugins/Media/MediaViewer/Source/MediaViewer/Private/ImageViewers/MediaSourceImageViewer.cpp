// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageViewers/MediaSourceImageViewer.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "Brushes/SlateImageBrush.h"
#include "DetailLayoutBuilder.h"
#include "Engine/Texture2D.h"
#include "FileMediaSource.h"
#include "IMediaStreamPlayer.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "Math/Box2D.h"
#include "Math/Vector2D.h"
#include "MediaPlayer.h"
#include "MediaSource.h"
#include "MediaStream.h"
#include "MediaStreamSourceBlueprintLibrary.h"
#include "MediaTexture.h"
#include "MediaTextureTracker.h"
#include "MediaViewerUtils.h"
#include "Misc/Optional.h"
#include "Misc/TVariant.h"
#include "Providers/ViewerTileVisibilityProvider.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/MediaImageStatusBarExtender.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SMediaSourceOverlay.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaSourceImageViewer)

#define LOCTEXT_NAMESPACE "MediaSourceImageViewer"

namespace UE::MediaViewer::Private
{

constexpr const TCHAR* MediaSourceThumbnailTexturePath = TEXT("/Script/Engine.Texture2D'/Engine/EditorResources/SceneManager.SceneManager'");

UTexture2D* GetMediaSourceThumbnailTexture()
{
	return LoadObject<UTexture2D>(GetTransientPackage(), MediaSourceThumbnailTexturePath);
}

const FLazyName FMediaSourceImageViewer::ItemTypeName_Asset = TEXT("MediaAsset");
const FLazyName FMediaSourceImageViewer::ItemTypeName_File = TEXT("MediaFile");

bool FMediaSourceImageViewer::FFactory::SupportsAsset(const FAssetData& InAssetData) const
{
	if (UClass* Class = InAssetData.GetClass(EResolveClass::Yes))
	{
		return Class->IsChildOf<UMediaSource>();
	}

	return false;
}

TSharedPtr<FMediaImageViewer> FMediaSourceImageViewer::FFactory::CreateImageViewer(const FAssetData& InAssetData) const
{
	if (UObject* Object = InAssetData.GetAsset())
	{
		return CreateImageViewer(TNotNull<UObject*>(Object));
	}

	return nullptr;
}

TSharedPtr<FMediaViewerLibraryItem> FMediaSourceImageViewer::FFactory::CreateLibraryItem(const FAssetData& InAssetData) const
{
	if (UObject* Object = InAssetData.GetAsset())
	{
		return CreateLibraryItem(TNotNull<UObject*>(Object));
	}

	return nullptr;
}

bool FMediaSourceImageViewer::FFactory::SupportsObject(TNotNull<UObject*> InObject) const
{
	return InObject->IsA<UMediaSource>();
}

TSharedPtr<FMediaImageViewer> FMediaSourceImageViewer::FFactory::CreateImageViewer(TNotNull<UObject*> InObject) const
{
	if (UMediaSource* MediaSource = Cast<UMediaSource>(InObject))
	{
		return MakeShared<FMediaSourceImageViewer>(MediaSource, FMediaImageViewer::GetObjectDisplayName(MediaSource));
	}

	return nullptr;
}

TSharedPtr<FMediaViewerLibraryItem> FMediaSourceImageViewer::FFactory::CreateLibraryItem(TNotNull<UObject*> InObject) const
{
	if (UMediaSource* MediaSource = Cast<UMediaSource>(InObject))
	{
		return MakeShared<FMediaSourceImageViewer::FAssetItem>(
			FMediaImageViewer::GetObjectDisplayName(MediaSource),
			FText::Format(LOCTEXT("FactoryToolTipFormat", "{0} [Media Source]"), FText::FromString(MediaSource->GetPathName())),
			MediaSource->HasAnyFlags(EObjectFlags::RF_Transient) || MediaSource->IsIn(GetTransientPackage()),
			MediaSource
		);
	}

	return nullptr;	
}

bool FMediaSourceImageViewer::FFactory::SupportsItemType(FName InItemType) const
{
	return InItemType == FMediaSourceImageViewer::ItemTypeName_Asset;
}

TSharedPtr<FMediaViewerLibraryItem> FMediaSourceImageViewer::FFactory::CreateLibraryItem(const FMediaViewerLibraryItem& InSavedItem) const
{
	FAssetData AssetData;

	if (IAssetRegistry::Get()->TryGetAssetByObjectPath(InSavedItem.GetStringValue(), AssetData) == UE::AssetRegistry::EExists::Exists)
	{
		return MakeShared<FMediaSourceImageViewer::FAssetItem>(FPrivateToken(), InSavedItem);
	}

	return nullptr;
	
}

FMediaSourceImageViewer::FItem::FItem(const FText& InName, const FText& InToolTip, bool bInTransient, const FString& InStringValue)
	: FMediaSourceImageViewer::FItem::FItem(FGuid::NewGuid(), InName, InToolTip, bInTransient, InStringValue)
{

}

FMediaSourceImageViewer::FItem::FItem(const FGuid& InId, const FText& InName, const FText& InToolTip, bool bInTransient, const FString& InStringValue)
	: FMediaViewerLibraryItem(InId, InName, InToolTip, bInTransient, InStringValue)
{

}

TSharedPtr<FSlateBrush> FMediaSourceImageViewer::FItem::CreateThumbnail()
{
	// Loading the media to create a thumbnail is not a good idea.
	TSharedRef<FSlateImageBrush> ThumbnailBrush = MakeShared<FSlateImageBrush>(
		GetMediaSourceThumbnailTexture(),
		FVector2D(64.0, 64.0)
	);

	return ThumbnailBrush;
}

FString FMediaSourceImageViewer::FItem::GetReferencerName() const
{
	static const FString ReferencerName = TEXT("FMediaSourceImageViewer::FItem");
	return ReferencerName;
}

void FMediaSourceImageViewer::FItem::AddReferencedObjects(FReferenceCollector& InCollector)
{
	if (Texture)
	{
		InCollector.AddReferencedObject(Texture);
	}
}

FMediaSourceImageViewer::FAssetItem::FAssetItem(const FText& InName, const FText& InToolTip, bool bInTransient, TNotNull<UMediaSource*> InMediaSource)
	: FMediaSourceImageViewer::FAssetItem(FGuid::NewGuid(), InName, InToolTip, bInTransient, InMediaSource)
{
}

FMediaSourceImageViewer::FAssetItem::FAssetItem(const FGuid& InId, const FText& InName, const FText& InToolTip, bool bInTransient,
	TNotNull<UMediaSource*> InMediaSource)
	: FItem(InId, InName, InToolTip, bInTransient, InMediaSource->GetPathName())
{
}

FMediaSourceImageViewer::FAssetItem::FAssetItem(FPrivateToken&& InPrivateToken, const FMediaViewerLibraryItem& InItem)
	: FMediaSourceImageViewer::FItem(InItem.GetId(), InItem.Name, InItem.ToolTip, InItem.IsTransient(), InItem.GetStringValue())
{
}

FName FMediaSourceImageViewer::FAssetItem::GetItemType() const
{
	return FMediaSourceImageViewer::ItemTypeName_Asset;
}

FSlateColor FMediaSourceImageViewer::FItem::GetItemTypeColor() const
{
	return GetClassColor(UFileMediaSource::StaticClass());
}

FText FMediaSourceImageViewer::FAssetItem::GetItemTypeDisplayName() const
{
	return LOCTEXT("MediaSource", "Media Asset");
}

TSharedPtr<FMediaImageViewer> FMediaSourceImageViewer::FAssetItem::CreateImageViewer() const
{
	UMediaSource* MediaSource = LoadAssetFromString<UMediaSource>(StringValue);

	if (!MediaSource)
	{
		return nullptr;
	}

	const FText DisplayName = FMediaImageViewer::GetObjectDisplayName(MediaSource);

	if (Id.IsValid())
	{
		return MakeShared<FMediaSourceImageViewer>(Id, MediaSource, DisplayName);
	}

	return MakeShared<FMediaSourceImageViewer>(MediaSource, DisplayName);
}

TSharedPtr<FMediaViewerLibraryItem> FMediaSourceImageViewer::FAssetItem::Clone() const
{
	if (StringValue.IsEmpty())
	{
		return nullptr;
	}

	return MakeShared<FMediaSourceImageViewer::FAssetItem>(FPrivateToken(), *this);
}

TOptional<FAssetData> FMediaSourceImageViewer::FAssetItem::AsAsset() const
{
	if (UMediaSource* MediaSource = LoadAssetFromString<UMediaSource>(StringValue))
	{
		if (MediaSource->IsAsset())
		{
			return FAssetData(MediaSource);
		}
	}

	return {};
}

FMediaSourceImageViewer::FExternalItem::FExternalItem(const FText& InName, const FText& InToolTip, const FString& InFilePath)
	: FMediaSourceImageViewer::FExternalItem(FGuid::NewGuid(), InName, InToolTip, InFilePath)
{
}

FMediaSourceImageViewer::FExternalItem::FExternalItem(const FGuid& InId, const FText& InName, const FText& InToolTip, const FString& InFilePath)
	: FMediaSourceImageViewer::FItem(InId, InName, InToolTip, /* Transient */ false, InFilePath)
{
}

FMediaSourceImageViewer::FExternalItem::FExternalItem(FPrivateToken&& InPrivateToken, const FMediaViewerLibraryItem& InItem)
	: FMediaSourceImageViewer::FItem(InItem.GetId(), InItem.Name, InItem.ToolTip, InItem.IsTransient(), InItem.GetStringValue())
{
}

FName FMediaSourceImageViewer::FExternalItem::GetItemType() const
{
	return FMediaSourceImageViewer::ItemTypeName_File;
}

FText FMediaSourceImageViewer::FExternalItem::GetItemTypeDisplayName() const
{
	return LOCTEXT("MediaFile", "Media File");
}

TSharedPtr<FMediaImageViewer> FMediaSourceImageViewer::FExternalItem::CreateImageViewer() const
{
	UFileMediaSource* FileMediaSource = NewObject<UFileMediaSource>(GetTransientPackage());
	FileMediaSource->SetFilePath(StringValue);

	const FText DisplayName = FText::FromString(StringValue);

	if (Id.IsValid())
	{
		return MakeShared<FMediaSourceImageViewer>(Id, FileMediaSource, DisplayName);
	}

	return MakeShared<FMediaSourceImageViewer>(FileMediaSource, DisplayName);
}

TSharedPtr<FMediaViewerLibraryItem> FMediaSourceImageViewer::FExternalItem::Clone() const
{
	if (StringValue.IsEmpty())
	{
		return nullptr;
	}

	return MakeShared<FMediaSourceImageViewer::FExternalItem>(FPrivateToken(), *this);
}

FMediaSourceImageViewer::FMediaSourceImageViewer(TNotNull<UMediaSource*> InMediaSource, const FText& InDisplayName)
	: FMediaSourceImageViewer(FGuid::NewGuid(), InMediaSource, InDisplayName)
{
}

FMediaSourceImageViewer::FMediaSourceImageViewer(const FGuid& InId, TNotNull<UMediaSource*> InMediaSource, const FText& InDisplayName)
	: FMediaImageViewer({
		InId,
		FIntPoint(1, 1),
		1,
		InDisplayName
	})
{
	MediaSourceSettings.MediaSource = InMediaSource;

	MediaStream = NewObject<UMediaStream>(GetTransientPackage());

	TSoftObjectPtr<UObject> SoftPtr(InMediaSource);
	FMediaStreamSource Source = UMediaStreamSourceBlueprintLibrary::MakeMediaSourceFromAsset(MediaStream, SoftPtr);

	if (UMediaStreamSourceBlueprintLibrary::IsValidMediaSource(Source))
	{
		// Register a tile-visibility provider before the media player opens.
		TSharedPtr<FViewerTileVisibilityProvider, ESPMode::ThreadSafe> LocalProvider;
		TSharedPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe> LocalTrackerObject;
		TWeakObjectPtr<UMediaTexture> LocalRegisteredTexture;

		const FDelegateHandle PreSourceChangeHandle = MediaStream->GetOnPreSourceChange().AddLambda(
			[&LocalProvider, &LocalTrackerObject, &LocalRegisteredTexture](UMediaStream* InMediaStream)
			{
				if (!InMediaStream)
				{
					return;
				}

				const TScriptInterface<IMediaStreamPlayer> Player = InMediaStream->GetPlayer();
				UMediaTexture* MediaTexture = Player ? Player->GetMediaTexture() : nullptr;
				if (!MediaTexture)
				{
					return;
				}

				LocalProvider = MakeShared<FViewerTileVisibilityProvider, ESPMode::ThreadSafe>();

				LocalTrackerObject = MakeShared<FMediaTextureTrackerObject, ESPMode::ThreadSafe>();
				LocalTrackerObject->TileVisibilityProvider = LocalProvider;

				LocalRegisteredTexture = MediaTexture;
				FMediaTextureTracker::Get().RegisterTexture(LocalTrackerObject, MediaTexture);
			});

		MediaStream->SetSource(Source);

		// Synchronous: SetSource above has already fired (or skipped) the hook; drop the subscription.
		MediaStream->GetOnPreSourceChange().Remove(PreSourceChangeHandle);

		// Adopt whatever the synchronous broadcast produced.
		TileVisibilityProvider = MoveTemp(LocalProvider);
		TextureTrackerObject = MoveTemp(LocalTrackerObject);
		RegisteredMediaTexture = LocalRegisteredTexture;

		if (TScriptInterface<IMediaStreamPlayer> Player = MediaStream->GetPlayer())
		{
			FMediaStreamTextureConfig TextureConfig = Player->GetTextureConfig();
			TextureConfig.bEnableMipGen = true;

			Player->SetTextureConfig(TextureConfig);

			if (UMediaTexture* MediaTexture = Player->GetMediaTexture())
			{
				MediaTexture->MipGenSettings = TextureMipGenSettings::TMGS_Unfiltered;

				ImageInfo.Size.X = MediaTexture->GetSurfaceWidth();
				ImageInfo.Size.Y = MediaTexture->GetSurfaceHeight();

				FMediaStreamPlayerConfig Config = Player->GetPlayerConfig();
				Config.bPlayOnOpen = true;
				Config.bLooping = true;
				Player->SetPlayerConfig(Config);

				Brush = MakeShared<FSlateImageBrush>(MediaTexture, FVector2D(ImageInfo.Size.X, ImageInfo.Size.Y));

				// Make sure it doesn't display in the media texture list
				MediaTexture->AddAssetUserData(NewObject<UMediaViewerUserData>(MediaTexture));

				// We can't know the pixel format of the media texture at this point.
				SampleCache = MakeShared<FTextureSampleCache>(MediaTexture, PF_Unknown);
			}
		}
	}
}

FMediaSourceImageViewer::~FMediaSourceImageViewer()
{
	if (UObjectInitialized() && MediaStream)
	{
		MediaStream->Close();
	}
	
	if (TextureTrackerObject.IsValid())
	{
		// A null RegisteredMediaTexture (GC'd before this dtor) is fine - UnregisterTexture handles null.
		FMediaTextureTracker::Get().UnregisterTexture(TextureTrackerObject, RegisteredMediaTexture.Get());
		TextureTrackerObject.Reset();
	}
	TileVisibilityProvider.Reset();
}

UMediaStream* FMediaSourceImageViewer::GetMediaStream() const
{
	return MediaStream;
}

void FMediaSourceImageViewer::UpdateTileVisibilityViewport(FIntPoint InDisplayRectPx, float InDPIScale)
{
	if (!TileVisibilityProvider.IsValid())
	{
		return;
	}
	if (InDisplayRectPx.X <= 0 || InDisplayRectPx.Y <= 0
		|| ImageInfo.Size.X <= 0 || ImageInfo.Size.Y <= 0)
	{
		return;
	}

	const FMediaImagePaintSettings& Paint = GetPaintSettings();

	// Lift slate-unit values into physical pixels (DPI = slate -> physical ratio).
	const double DPI = InDPIScale > 0.0f ? static_cast<double>(InDPIScale) : 1.0;
	const double ZoomScale = static_cast<double>(FMath::Max(Paint.Scale, KINDA_SMALL_NUMBER));
	const FVector2D SourceDim(ImageInfo.Size);
	const FVector2D PaintSizePx = SourceDim * ZoomScale * DPI;
	const FVector2D UserOffsetPx(Paint.Offset.X * DPI, Paint.Offset.Y * DPI);

	// Where the painted image's top-left lands in the panel, in physical pixels (mirrors
	// FMediaImageViewer::GetPaintOffset). Negative when the image is bigger than the panel
	// or panned off the left/top edge.
	const FVector2D PanelCenter = FVector2D(InDisplayRectPx) * 0.5;
	const FVector2D PaintTopLeft = PanelCenter - PaintSizePx * 0.5 + UserOffsetPx;

	// Visible UV window of the source after pan/zoom, clamped to [0, 1].
	const FVector2D UvMin(
		FMath::Clamp((0.0 - PaintTopLeft.X) / PaintSizePx.X, 0.0, 1.0),
		FMath::Clamp((0.0 - PaintTopLeft.Y) / PaintSizePx.Y, 0.0, 1.0));
	const FVector2D UvMax(
		FMath::Clamp((static_cast<double>(InDisplayRectPx.X) - PaintTopLeft.X) / PaintSizePx.X, 0.0, 1.0),
		FMath::Clamp((static_cast<double>(InDisplayRectPx.Y) - PaintTopLeft.Y) / PaintSizePx.Y, 0.0, 1.0));

	FViewerTileVisibilityProvider::FUpdateInputs Inputs;
	Inputs.VisibleSourceRect = FBox2D(UvMin * SourceDim, UvMax * SourceDim);
	Inputs.DisplayedSizePx = (UvMax - UvMin) * PaintSizePx;
	if (const TOptional<uint8> PaintMipOverride = Paint.GetMipOverride())
	{
		Inputs.MipOverride = *PaintMipOverride;
	}
	TileVisibilityProvider->Update(Inputs);
}

TSharedPtr<FMediaViewerLibraryItem> FMediaSourceImageViewer::CreateLibraryItem() const
{
	UMediaSource* MediaSource = MediaSourceSettings.MediaSource;

	if (!MediaSource)
	{
		return nullptr;
	}

	const bool bIsTransient = MediaSource->HasAnyFlags(EObjectFlags::RF_Transient) || MediaSource->IsIn(GetTransientPackage());
	UFileMediaSource* FileMediaSource = Cast<UFileMediaSource>(MediaSource);

	if (!bIsTransient || !FileMediaSource)
	{
		return MakeShared<FAssetItem>(
			ImageInfo.Id,
			FMediaImageViewer::GetObjectDisplayName(MediaSource),
			FText::Format(LOCTEXT("LibraryToolTipFormat", "{0} [Media Texture - {1}x{2}]"), FText::FromString(MediaSource->GetPathName()), FText::AsNumber(ImageInfo.Size.X), FText::AsNumber(ImageInfo.Size.Y)),
			bIsTransient,
			MediaSource
		);
	}

	return MakeShared<FExternalItem>(
		ImageInfo.Id,
		FText::FromString(FileMediaSource->GetFilePath()),
		FText::Format(LOCTEXT("LibraryToolTipFormat", "{0} [Media Texture - {1}x{2}]"), FText::FromString(FileMediaSource->GetFilePath()), FText::AsNumber(ImageInfo.Size.X), FText::AsNumber(ImageInfo.Size.Y)),
		FileMediaSource->GetFilePath()
	);
}

TOptional<TVariant<FColor, FLinearColor>> FMediaSourceImageViewer::GetPixelColor(const FIntPoint& InPixelCoords, int32 InMipLevel) const
{
	if (!SampleCache.IsValid() || !SampleCache->IsValid())
	{
		return {};
	}
	
	if (InPixelCoords.X < 0 || InPixelCoords.Y < 0)
	{
		SampleCache->Invalidate();
		return {};
	}

	if (InPixelCoords.X >= ImageInfo.Size.X || InPixelCoords.Y >= ImageInfo.Size.Y)
	{
		SampleCache->Invalidate();
		return {};
	}

	FIntPoint MirroredPixelCoords = InPixelCoords;

	if (EnumHasAllFlags(static_cast<EMediaImageMirrorPlane>(PaintSettings.MirrorPlanes), EMediaImageMirrorPlane::Horizontal))
	{
		MirroredPixelCoords.X = ImageInfo.Size.X - InPixelCoords.X - 1;
	}

	if (EnumHasAllFlags(static_cast<EMediaImageMirrorPlane>(PaintSettings.MirrorPlanes), EMediaImageMirrorPlane::Vertical))
	{
		MirroredPixelCoords.Y = ImageInfo.Size.Y - InPixelCoords.Y - 1;
	}

	if (!MediaStream)
	{
		SampleCache->Invalidate();
		return {};
	}

	IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface();

	if (!MediaStreamPlayer)
	{
		SampleCache->Invalidate();
		return {};
	}

	if (UMediaPlayer* Player = MediaStreamPlayer->GetPlayer())
	{
		const FTimespan PlayerTime = Player->GetTime();

		// Sample from the mip the visibility provider is actually loading. InMipLevel
		// reflects only the paint-settings override (0 by default) and ignores LOD-derived
		// mips, so a provider-driven source would otherwise read stale mip 0 data.
		uint8 SampleMipLevel = static_cast<uint8>(FMath::Max(0, InMipLevel));
		if (TileVisibilityProvider.IsValid())
		{
			const int32 ProviderMip = TileVisibilityProvider->GetDisplayedMipFloor();
			if (ProviderMip != INDEX_NONE)
			{
				SampleMipLevel = static_cast<uint8>(FMath::Clamp(ProviderMip, 0, MAX_uint8));
			}
		}

		if (const FLinearColor* PixelColor = SampleCache->GetPixelColor(MirroredPixelCoords, SampleMipLevel, PlayerTime))
		{
			TVariant<FColor, FLinearColor> PixelColorVariant;
			PixelColorVariant.Set<FLinearColor>(*PixelColor);

			return PixelColorVariant;
		}
	}

	return {};
}

TSharedPtr<FStructOnScope> FMediaSourceImageViewer::GetCustomSettingsOnScope() const
{
	return MakeShared<FStructOnScope>(
		FMediaSourceImageViewerSettings::StaticStruct(),
		reinterpret_cast<uint8*>(&const_cast<FMediaSourceImageViewerSettings&>(MediaSourceSettings))
	);
}

void FMediaSourceImageViewer::AddReferencedObjects(FReferenceCollector& InCollector)
{
	FMediaImageViewer::AddReferencedObjects(InCollector);

	InCollector.AddPropertyReferencesWithStructARO(
		FMediaSourceImageViewerSettings::StaticStruct(),
		&MediaSourceSettings
	);

	if (MediaStream)
	{
		InCollector.AddReferencedObject(MediaStream);
	}
}

void FMediaSourceImageViewer::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged)
{
	FMediaImageViewer::NotifyPostChange(InPropertyChangedEvent, InPropertyThatChanged);

	if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(FMediaImagePaintSettings, MirrorPlanes))
	{
		OnMirrored();
	}
}

void FMediaSourceImageViewer::OnMirrored()
{
	if (SampleCache.IsValid())
	{
		SampleCache->Invalidate();
	}
}

TSharedPtr<SWidget> FMediaSourceImageViewer::GetOverlayWidget(EMediaImageViewerPosition InPosition, const TSharedPtr<FMediaViewerDelegates>& InDelegates)
{
	return SNew(SBox)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Bottom)
		.Padding(5.f)
		[
			SNew(SMediaSourceOverlay, SharedThis(this), InPosition, InDelegates)
		];
}

void FMediaSourceImageViewer::ExtendStatusBar(UE::MediaViewer::FMediaImageStatusBarExtender& InOutStatusBarExtender)
{
	InOutStatusBarExtender.AddExtension(
		UE::MediaViewer::StatusBarSections::StatusBarCenter,
		EExtensionHook::Before,
		nullptr,
		UE::MediaViewer::FMediaImageStatusBarExtension::FDelegate::CreateSP(this, &FMediaSourceImageViewer::AddPlayerName)
	);
}

void FMediaSourceImageViewer::AddPlayerName(const TSharedRef<SHorizontalBox>& InStatusBar)
{
	if (!MediaStream)
	{
		return;
	}

	IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface();

	if (!MediaStreamPlayer)
	{
		return;
	}

	UMediaPlayer* MediaPlayer = MediaStreamPlayer->GetPlayer();

	if (!MediaPlayer)
	{
		return;
	}

	const FName PlayerName = MediaPlayer->GetPlayerName();

	if (PlayerName.IsNone())
	{
		return;
	}

	InStatusBar->AddSlot()
		.AutoWidth()
		.Padding(2.f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(FText::FromName(PlayerName))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];
}

bool FMediaSourceImageViewer::OnOwnerCleanup(UObject* InObject)
{
	bool bInvalidate = false;

	if (InObject)
	{
		if (MediaSourceSettings.MediaSource && 
			(MediaSourceSettings.MediaSource == InObject || MediaSourceSettings.MediaSource->IsIn(InObject)))
		{
			bInvalidate = true;
		}

		if (MediaStream && (MediaStream == InObject || MediaStream->IsIn(InObject)))
		{
			bInvalidate = true;
		}
	}

	if (bInvalidate)
	{
		SampleCache.Reset();

		if (Brush.IsValid())
		{
			Brush->SetResourceObject(nullptr);
		}
	}

	return bInvalidate;
}

FString FMediaSourceImageViewer::GetReferencerName() const
{
	static const FString ReferencerName = TEXT("FMediaSourceImageViewer");
	return ReferencerName;
}

void FMediaSourceImageViewer::PaintImage(FMediaImageSlatePaintParams& InPaintParams, const FMediaImageSlatePaintGeometry& InPaintGeometry)
{
	if (TScriptInterface<IMediaStreamPlayer> Player = MediaStream->GetPlayer())
	{
		if (UMediaTexture* MediaTexture = Player->GetMediaTexture())
		{
			ImageInfo.Size.X = MediaTexture->GetSurfaceWidth();
			ImageInfo.Size.Y = MediaTexture->GetSurfaceHeight();

			MediaTexture->MipGenSettings = MediaSourceSettings.MipGenType;
		}
	}

	// Push the actual image-display rect into the tile-visibility provider. ViewerSize is the
	// image area in slate units; the provider expects post-DPI pixels, so multiply by DPIScale.
	// Sourcing this from PaintImage (rather than the overlay widget's own geometry) is essential -
	// the overlay is just the small bottom strip of controls, so its size would clip the visible
	// rect to a thin horizontal slice and starve the upper/lower tiles.
	{
		const float DPI = InPaintParams.DPIScale > 0.0f ? InPaintParams.DPIScale : 1.0f;
		const FIntPoint DisplayRectPx(
			FMath::Max(0, FMath::CeilToInt32(InPaintParams.ViewerSize.X * DPI)),
			FMath::Max(0, FMath::CeilToInt32(InPaintParams.ViewerSize.Y * DPI)));
		UpdateTileVisibilityViewport(DisplayRectPx, DPI);
	}

	// Always invalidate the mip texture for video.
	bInvalidatedMipTexture = true;

	FMediaImageViewer::PaintImage(InPaintParams, InPaintGeometry);
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
