// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImageViewer/IMediaImageViewerFactory.h"
#include "ImageViewer/MediaImageViewer.h"
#include "Library/MediaViewerLibraryItem.h"

#include "Engine/TextureDefines.h"
#include "ImageViewers/TextureSampleCache.h"
#include "Math/Color.h"
#include "Misc/Optional.h"
#include "Misc/Timespan.h"
#include "Templates/SharedPointer.h"

#include "MediaSourceImageViewer.generated.h"

class SHorizontalBox;
class UMediaSource;
class UMediaStream;
class UMediaTexture;
struct FMediaTextureTrackerObject;

USTRUCT()
struct FMediaSourceImageViewerSettings
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Texture")
	TObjectPtr<UMediaSource> MediaSource = nullptr;

	UPROPERTY(VisibleAnywhere, Category = "Texture")
	TEnumAsByte<TextureMipGenSettings> MipGenType = TextureMipGenSettings::TMGS_Unfiltered;
};

namespace UE::MediaViewer::Private
{

class FViewerTileVisibilityProvider;

class FMediaSourceImageViewer : public FMediaImageViewer
{
public:
	struct FFactory : public IMediaImageViewerFactory
	{
		FFactory()
		{
			Priority = 5000;
		}

		virtual bool SupportsAsset(const FAssetData& InAssetData) const override;
		virtual TSharedPtr<FMediaImageViewer> CreateImageViewer(const FAssetData& InAssetData) const override;
		virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(const FAssetData& InAssetData) const override;

		virtual bool SupportsObject(TNotNull<UObject*> InObject) const override;
		virtual TSharedPtr<FMediaImageViewer> CreateImageViewer(TNotNull<UObject*> InObject) const override;
		virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(TNotNull<UObject*> InObject) const override;

		virtual bool SupportsItemType(FName InItemType) const override;
		virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(const FMediaViewerLibraryItem& InSavedItem) const override;
	};

	struct FItem : public FMediaViewerLibraryItem, public FGCObject
	{
		TObjectPtr<UTexture> Texture;

		FItem(const FText& InName, const FText& InToolTip, bool bInTransient, const FString& InStringValue);

		FItem(const FGuid& InId, const FText& InName, const FText& InToolTip, bool bInTransient, const FString& InStringValue);

		//~ Begin FMediaViewerLibraryItem
		virtual FSlateColor GetItemTypeColor() const override;
		virtual TSharedPtr<FSlateBrush> CreateThumbnail() override;
		//~ End FMediaViewerLibraryItem

		//~ Begin FGCObject
		virtual FString GetReferencerName() const override;
		virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
		//~ End FGCObject
	};

	struct FAssetItem : public FItem
	{
		FAssetItem(const FText& InName, const FText& InToolTip, bool bInTransient, TNotNull<UMediaSource*> InMediaSource);

		FAssetItem(const FGuid& InId, const FText& InName, const FText& InToolTip, bool bInTransient, TNotNull<UMediaSource*> InMediaSource);

		FAssetItem(FPrivateToken&& InPrivateToken, const FMediaViewerLibraryItem& InItem);

		virtual ~FAssetItem() override = default;

		//~ Begin FMediaViewerLibraryItem
		virtual FName GetItemType() const override;
		virtual FText GetItemTypeDisplayName() const override;
		virtual TSharedPtr<FMediaImageViewer> CreateImageViewer() const override;
		virtual TSharedPtr<FMediaViewerLibraryItem> Clone() const override;
		virtual TOptional<FAssetData> AsAsset() const override;
		//~ End FMediaViewerLibraryItem
	};

	struct FExternalItem : public FItem
	{
		FExternalItem(const FText& InName, const FText& InToolTip, const FString& InFilePath);

		FExternalItem(const FGuid& InId, const FText& InName, const FText& InToolTip, const FString& InFilePath);

		FExternalItem(FPrivateToken&& InPrivateToken, const FMediaViewerLibraryItem& InItem);

		virtual ~FExternalItem() override = default;

		//~ Begin FMediaViewerLibraryItem
		virtual FName GetItemType() const override;
		virtual FText GetItemTypeDisplayName() const override;
		virtual TSharedPtr<FMediaImageViewer> CreateImageViewer() const override;
		virtual TSharedPtr<FMediaViewerLibraryItem> Clone() const override;
		//~ End FMediaViewerLibraryItem
	};

	static const FLazyName ItemTypeName_Asset;
	static const FLazyName ItemTypeName_File;

	FMediaSourceImageViewer(TNotNull<UMediaSource*> InMediaSource, const FText& InDisplayName);
	FMediaSourceImageViewer(const FGuid& InId, TNotNull<UMediaSource*> InMediaSource, const FText& InDisplayName);

	virtual ~FMediaSourceImageViewer() override;

	UMediaStream* GetMediaStream() const;

	/**
	 * Push the latest panel viewport state to the tile-visibility provider so that the
	 * underlying tiled-media reader (e.g. ImgMedia EXR) requests only the tiles needed
	 * for what's actually visible on screen. Called by SMediaSourceOverlay::Tick.
	 *
	 * @param InDisplayRectPx Image area in absolute (post-DPI) screen pixels.
	 * @param InDPIScale      Slate-to-physical DPI scale of the panel.
	 */
	void UpdateTileVisibilityViewport(FIntPoint InDisplayRectPx, float InDPIScale);

	//~ Begin FMediaImageViewer
	virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem() const override;
	virtual bool OnOwnerCleanup(UObject* InObject) override;
	virtual TOptional<TVariant<FColor, FLinearColor>> GetPixelColor(const FIntPoint& InPixelCoords, int32 InMipLevel) const override;
	virtual TSharedPtr<FStructOnScope> GetCustomSettingsOnScope() const override;
	virtual void OnMirrored() override;
	virtual TSharedPtr<SWidget> GetOverlayWidget(EMediaImageViewerPosition InPosition, const TSharedPtr<FMediaViewerDelegates>& InDelegates) override;
	virtual void ExtendStatusBar(UE::MediaViewer::FMediaImageStatusBarExtender& InOutStatusBarExtender) override;
	//~ End FMediaImageViewer

	//~ Begin FGCObject
	virtual FString GetReferencerName() const override;
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	//~ End FGCObject

	//~ Begin FNotifyHook
	virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged) override;
	//~ End FNotifyHook

protected:
	FMediaSourceImageViewerSettings MediaSourceSettings;
	TObjectPtr<UMediaStream> MediaStream;
	TSharedPtr<FTextureSampleCache> SampleCache;

	/** Tile-visibility provider exposed via FMediaTextureTracker; the widget pushes viewport state into it each Tick. */
	TSharedPtr<FViewerTileVisibilityProvider, ESPMode::ThreadSafe> TileVisibilityProvider;

	/** FMediaTextureTracker bookkeeping for the provider. Registered against the player's MediaTexture. */
	TSharedPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe> TextureTrackerObject;

	/** The MediaTexture we registered with on the tracker - kept so the dtor can unregister against the same one. */
	TWeakObjectPtr<UMediaTexture> RegisteredMediaTexture;

	void AddPlayerName(const TSharedRef<SHorizontalBox>& InStatusBar);

	//~ Begin FMediaImageViewer
	virtual void PaintImage(FMediaImageSlatePaintParams& InPaintParams, const FMediaImageSlatePaintGeometry& InPaintGeometry) override;
	//~ End FMediaImageViewer
};

} // UE::MediaViewer::Private
