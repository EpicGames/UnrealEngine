// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageViewers/MediaTextureImageViewer.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "Brushes/SlateImageBrush.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "Math/Vector2D.h"
#include "MediaTexture.h"
#include "MediaViewerUtils.h"
#include "Misc/Optional.h"
#include "Misc/TVariant.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaTextureImageViewer)

#define LOCTEXT_NAMESPACE "MediaTextureImageViewer"

namespace UE::MediaViewer::Private
{

const FLazyName FMediaTextureImageViewer::ItemTypeName = TEXT("MediaTexture");

bool FMediaTextureImageViewer::FFactory::SupportsAsset(const FAssetData& InAssetData) const
{
	if (UClass* Class = InAssetData.GetClass(EResolveClass::Yes))
	{
		return Class->IsChildOf<UMediaTexture>();
	}

	return false;
}

TSharedPtr<FMediaImageViewer> FMediaTextureImageViewer::FFactory::CreateImageViewer(const FAssetData& InAssetData) const
{
	if (UObject* Object = InAssetData.GetAsset())
	{
		return CreateImageViewer(TNotNull<UObject*>(Object));
	}

	return nullptr;
}

TSharedPtr<FMediaViewerLibraryItem> FMediaTextureImageViewer::FFactory::CreateLibraryItem(const FAssetData& InAssetData) const
{
	if (UObject* Object = InAssetData.GetAsset())
	{
		return CreateLibraryItem(TNotNull<UObject*>(Object));
	}

	return nullptr;
}

bool FMediaTextureImageViewer::FFactory::SupportsObject(TNotNull<UObject*> InObject) const
{
	return InObject->IsA<UMediaTexture>();
}

TSharedPtr<FMediaImageViewer> FMediaTextureImageViewer::FFactory::CreateImageViewer(TNotNull<UObject*> InObject) const
{
	if (UMediaTexture* MediaTexture = Cast<UMediaTexture>(InObject))
	{
		return MakeShared<FMediaTextureImageViewer>(MediaTexture);
	}

	return nullptr;
}

TSharedPtr<FMediaViewerLibraryItem> FMediaTextureImageViewer::FFactory::CreateLibraryItem(TNotNull<UObject*> InObject) const
{
	if (UMediaTexture* MediaTexture = Cast<UMediaTexture>(InObject))
	{
		return MakeShared<FMediaTextureImageViewer::FItem>(
			FMediaImageViewer::GetObjectDisplayName(InObject),
			FText::Format(LOCTEXT("ToolTipFromat", "{0} [Media Texture - {1}x{2}]"), FText::FromString(MediaTexture->GetPathName()), FText::AsNumber(MediaTexture->GetSurfaceWidth()), FText::AsNumber(MediaTexture->GetSurfaceHeight())),
			MediaTexture->HasAnyFlags(EObjectFlags::RF_Transient) || MediaTexture->IsIn(GetTransientPackage()),
			MediaTexture
		);
	}

	return nullptr;	
}

bool FMediaTextureImageViewer::FFactory::SupportsItemType(FName InItemType) const
{
	return InItemType == FMediaTextureImageViewer::ItemTypeName;
}

TSharedPtr<FMediaViewerLibraryItem> FMediaTextureImageViewer::FFactory::CreateLibraryItem(const FMediaViewerLibraryItem& InSavedItem) const
{
	FAssetData AssetData;

	if (IAssetRegistry::Get()->TryGetAssetByObjectPath(InSavedItem.GetStringValue(), AssetData) == UE::AssetRegistry::EExists::Exists)
	{
		return MakeShared<FMediaTextureImageViewer::FItem>(FPrivateToken(), InSavedItem);
	}

	return nullptr;
}

FMediaTextureImageViewer::FItem::FItem(const FText& InName, const FText& InToolTip, bool bInTransient, TNotNull<UMediaTexture*> InMediaTexture)
	: FItem(FGuid::NewGuid(), InName, InToolTip, bInTransient, InMediaTexture)
{
}

FMediaTextureImageViewer::FItem::FItem(const FGuid& InId, const FText& InName, const FText& InToolTip, bool bInTransient,
	TNotNull<UMediaTexture*> InMediaTexture)
	: FMediaViewerLibraryItem(InId, InName, InToolTip, bInTransient, InMediaTexture->GetPathName())
{
}

FMediaTextureImageViewer::FItem::FItem(FPrivateToken&& InPrivateToken, const FMediaViewerLibraryItem& InItem)
	: FMediaViewerLibraryItem(InItem.GetId(), InItem.Name, InItem.ToolTip, InItem.IsTransient(), InItem.GetStringValue())
{
}

FName FMediaTextureImageViewer::FItem::GetItemType() const
{
	return FMediaTextureImageViewer::ItemTypeName;
}

FText FMediaTextureImageViewer::FItem::GetItemTypeDisplayName() const
{
	return LOCTEXT("MediaTexture", "Media Texture");
}

FSlateColor FMediaTextureImageViewer::FItem::GetItemTypeColor() const
{
	return GetClassColor(UMediaTexture::StaticClass());
}

TSharedPtr<FSlateBrush> FMediaTextureImageViewer::FItem::CreateThumbnail()
{
	UMediaTexture* MediaTexture = LoadAssetFromString<UMediaTexture>(StringValue);

	if (!MediaTexture)
	{
		return nullptr;
	}

	TSharedRef<FSlateImageBrush> ThumbnailBrush = MakeShared<FSlateImageBrush>(MediaTexture, FVector2D(MediaTexture->GetSurfaceWidth(), MediaTexture->GetSurfaceDepth()));

	return ThumbnailBrush;
}

TSharedPtr<FMediaImageViewer> FMediaTextureImageViewer::FItem::CreateImageViewer() const
{
	UMediaTexture* MediaTexture = LoadAssetFromString<UMediaTexture>(StringValue);

	if (!MediaTexture)
	{
		return nullptr;
	}

	if (Id.IsValid())
	{
		return MakeShared<FMediaTextureImageViewer>(Id, MediaTexture);
	}

	return MakeShared<FMediaTextureImageViewer>(MediaTexture);
}

TSharedPtr<FMediaViewerLibraryItem> FMediaTextureImageViewer::FItem::Clone() const
{
	if (StringValue.IsEmpty())
	{
		return nullptr;
	}

	return MakeShared<FMediaTextureImageViewer::FItem>(FPrivateToken(), *this);
}

TOptional<FAssetData> FMediaTextureImageViewer::FItem::AsAsset() const
{
	if (UMediaTexture* MediaTexture = LoadAssetFromString<UMediaTexture>(StringValue))
	{
		if (MediaTexture->IsAsset())
		{
			return FAssetData(MediaTexture);
		}
	}

	return {};
}

FMediaTextureImageViewer::FMediaTextureImageViewer(TNotNull<UMediaTexture*> InMediaTexture)
	: FMediaTextureImageViewer(FGuid::NewGuid(), InMediaTexture)
{
}

FMediaTextureImageViewer::FMediaTextureImageViewer(const FGuid& InId, TNotNull<UMediaTexture*> InMediaTexture)
	: FMediaImageViewer({
		InId,
		FIntPoint(InMediaTexture->GetSurfaceWidth(), InMediaTexture->GetSurfaceHeight()),
		InMediaTexture->GetTextureNumMips(),
		FMediaImageViewer::GetObjectDisplayName(InMediaTexture)
	})
{
	MediaTextureSettings.MediaTexture = InMediaTexture;

	if (ImageInfo.Size.X < 2 || ImageInfo.Size.Y < 2)
	{
		return;
	}

	Brush = MakeShared<FSlateImageBrush>(
		InMediaTexture,
		FVector2D(ImageInfo.Size.X, ImageInfo.Size.Y)
	);

	// We can't know the pixel format of the media texture at this point.
	SampleCache = MakeShared<FTextureSampleCache>(InMediaTexture, PF_Unknown);
}

TSharedPtr<FMediaViewerLibraryItem> FMediaTextureImageViewer::CreateLibraryItem() const
{
	UMediaTexture* MediaTexture = MediaTextureSettings.MediaTexture;

	if (!MediaTexture)
	{
		return nullptr;
	}

	return MakeShared<FItem>(
		ImageInfo.Id,
		FMediaImageViewer::GetObjectDisplayName(MediaTexture),
		FText::Format(LOCTEXT("ToolTipFromat", "{0} [Media Texture - {1}x{2}]"), FText::FromString(MediaTexture->GetPathName()), FText::AsNumber(MediaTexture->GetSurfaceWidth()), FText::AsNumber(MediaTexture->GetSurfaceHeight())),
		MediaTexture->HasAnyFlags(EObjectFlags::RF_Transient) || MediaTexture->IsIn(GetTransientPackage()),
		MediaTexture
	);
}

TOptional<TVariant<FColor, FLinearColor>> FMediaTextureImageViewer::GetPixelColor(const FIntPoint& InPixelCoords, int32 InMipLevel) const
{
	if (!SampleCache.IsValid() || !SampleCache->IsValid())
	{
		return {};
	}

	// Always mark the sample cache dirty. We have no control over the state of the media texture.
	SampleCache->MarkDirty();

	if (InPixelCoords.X < 0 || InPixelCoords.Y < 0)
	{
		SampleCache->Invalidate();
		return {};
	}

	if (InPixelCoords.X >= ImageInfo.Size.X || InPixelCoords.Y >= ImageInfo.Size.Y)
	{
		SampleCache->Invalidate();
		return {};
	}

	FIntPoint MirroredPixelCoords = InPixelCoords;

	if (EnumHasAllFlags(static_cast<EMediaImageMirrorPlane>(PaintSettings.MirrorPlanes), EMediaImageMirrorPlane::Horizontal))
	{
		MirroredPixelCoords.X = ImageInfo.Size.X - InPixelCoords.X - 1;
	}

	if (EnumHasAllFlags(static_cast<EMediaImageMirrorPlane>(PaintSettings.MirrorPlanes), EMediaImageMirrorPlane::Vertical))
	{
		MirroredPixelCoords.Y = ImageInfo.Size.Y - InPixelCoords.Y - 1;
	}

	if (const FLinearColor* PixelColor = SampleCache->GetPixelColor(MirroredPixelCoords))
	{
		TVariant<FColor, FLinearColor> PixelColorVariant;
		PixelColorVariant.Set<FLinearColor>(FMediaViewerUtils::CorrectGamma(*PixelColor));

		return PixelColorVariant;
	}

	return {};
}

TSharedPtr<FStructOnScope> FMediaTextureImageViewer::GetCustomSettingsOnScope() const
{
	return MakeShared<FStructOnScope>(
		FMediaTextureImageViewerSettings::StaticStruct(),
		reinterpret_cast<uint8*>(&const_cast<FMediaTextureImageViewerSettings&>(MediaTextureSettings))
	);
}

void FMediaTextureImageViewer::AddReferencedObjects(FReferenceCollector& InCollector)
{
	FMediaImageViewer::AddReferencedObjects(InCollector);

	InCollector.AddPropertyReferencesWithStructARO(
		FMediaTextureImageViewerSettings::StaticStruct(),
		&MediaTextureSettings
	);
}

void FMediaTextureImageViewer::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged)
{
	FMediaImageViewer::NotifyPostChange(InPropertyChangedEvent, InPropertyThatChanged);

	if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(FMediaImagePaintSettings, MirrorPlanes))
	{
		OnMirrored();
	}
}

void FMediaTextureImageViewer::OnMirrored()
{
	if (SampleCache.IsValid())
	{
		SampleCache->Invalidate();
	}
}

bool FMediaTextureImageViewer::OnOwnerCleanup(UObject* InObject)
{
	if (InObject && MediaTextureSettings.MediaTexture && 
		(MediaTextureSettings.MediaTexture == InObject || MediaTextureSettings.MediaTexture->IsIn(InObject)))
	{
		SampleCache.Reset();

		if (Brush.IsValid())
		{
			Brush->SetResourceObject(nullptr);
		}

		return true;
	}

	return false;
}

FString FMediaTextureImageViewer::GetReferencerName() const
{
	static const FString ReferencerName = TEXT("FMediaTextureImageViewer");
	return ReferencerName;
}

void FMediaTextureImageViewer::PaintImage(FMediaImageSlatePaintParams& InPaintParams, const FMediaImageSlatePaintGeometry& InPaintGeometry)
{
	// Always invalidate the mip texture for video.
	bInvalidatedMipTexture = true;

	FMediaImageViewer::PaintImage(InPaintParams, InPaintGeometry);
}
	
} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
