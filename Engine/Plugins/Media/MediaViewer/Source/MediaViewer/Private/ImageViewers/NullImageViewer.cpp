// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageViewers/NullImageViewer.h"

#include "Math/Color.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Misc/TVariant.h"

#define LOCTEXT_NAMESPACE "NullImageViewer"

namespace UE::MediaViewer::Private
{

TSharedRef<FNullImageViewer> FNullImageViewer::GetNullImageViewer()
{
	static TSharedRef<FNullImageViewer> NullImageViewer = MakeShared<FNullImageViewer>();
	return NullImageViewer;
}

FNullImageViewer::FNullImageViewer()
	: FMediaImageViewer({
		FGuid(), 
		FIntPoint::ZeroValue, 
		0, 
		LOCTEXT("Null", "-")
	})
{
}

TOptional<TVariant<FColor, FLinearColor>> FNullImageViewer::GetPixelColor(const FIntPoint& InPixelCoords, int32 InMipLevel) const
{
	return {};
}

FString FNullImageViewer::GetReferencerName() const
{
	static const FString ReferencerName = TEXT("FNullImageViewer");
	return ReferencerName;
}

TSharedPtr<FMediaViewerLibraryItem> FNullImageViewer::CreateLibraryItem() const
{
	return nullptr;
}

void FNullImageViewer::PaintImage(FMediaImageSlatePaintParams& InPaintParams, const FMediaImageSlatePaintGeometry& InPaintGeometry)
{
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
