// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImageViewer/MediaImageViewer.h"

namespace UE::MediaViewer::Private
{

class FNullImageViewer : public FMediaImageViewer
{
public:
	static TSharedRef<FNullImageViewer> GetNullImageViewer();

	FNullImageViewer();
	virtual ~FNullImageViewer() = default;

	//~ Begin FMediaImageViewer
	virtual TOptional<TVariant<FColor, FLinearColor>> GetPixelColor(const FIntPoint& InPixelCoords, int32 InMipLevel) const override;
	//~ End FMediaImageViewer

	//~ Begin FGCObject
	virtual FString GetReferencerName() const override;
	//~ End FGCObject

protected:
	//~ Begin FMediaImageViewer
	virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem() const override;
	virtual void PaintImage(FMediaImageSlatePaintParams& InPaintParams, const FMediaImageSlatePaintGeometry& InPaintGeometry) override;
	//~ End FMediaImageViewer
};

} // UE::MediaViewer::Private
