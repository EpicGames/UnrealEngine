// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageViewers/SceneViewportImageViewer.h"

#include "Slate/SceneViewport.h"

namespace UE::MediaViewer::Private
{

FSceneViewportImageViewer::FSceneViewportImageViewer(const TSharedPtr<FSceneViewport>& InViewport, const FText& InDisplayName)
	: FSceneViewportImageViewer(FGuid::NewGuid(), InViewport, InDisplayName)
{
}

FSceneViewportImageViewer::FSceneViewportImageViewer(const FGuid& InId, const TSharedPtr<FSceneViewport>& InViewport, const FText& InDisplayName)
	: FViewportImageViewer({
		InId,
		InViewport.IsValid() ? InViewport->GetSize() : FIntPoint(0, 0),
		/* Mip Count */ 1,
		InDisplayName
	})
	, ViewportWeak(InViewport)
{
	CreateBrush();
}

FViewport* FSceneViewportImageViewer::GetViewport() const
{
	return ViewportWeak.Pin().Get();
}

} // UE::MediaViewer::Private
