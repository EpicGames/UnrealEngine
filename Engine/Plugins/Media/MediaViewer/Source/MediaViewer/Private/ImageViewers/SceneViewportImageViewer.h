// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImageViewers/ViewportImageViewer.h"

class FSceneViewport;

namespace UE::MediaViewer::Private
{

class FSceneViewportImageViewer : public FViewportImageViewer
{
public:
	FSceneViewportImageViewer(const TSharedPtr<FSceneViewport>& InViewport, const FText& InName);
	FSceneViewportImageViewer(const FGuid& InId, const TSharedPtr<FSceneViewport>& InViewport, const FText& InDisplayName);

protected:
	TWeakPtr<FSceneViewport> ViewportWeak;

	//~ Begin FViewportImageViewer
	virtual FViewport* GetViewport() const override;
	//~ End FViewportImageViewer
};

} // UE::MediaViewer::Private
