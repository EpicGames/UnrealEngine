// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageViewers/Texture2DImageViewer.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "Brushes/SlateImageBrush.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "Math/Color.h"
#include "Math/IntPoint.h"
#include "Math/Vector2D.h"
#include "MediaViewerUtils.h"
#include "Misc/TVariant.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Texture2DImageViewer)

#define LOCTEXT_NAMESPACE "Texture2DImageViewer"

namespace UE::MediaViewer::Private
{

const FLazyName FTexture2DImageViewer::ItemTypeName = TEXT("Texture2D");

bool FTexture2DImageViewer::FFactory::SupportsAsset(const FAssetData& InAssetData) const
{
	if (UClass* Class = InAssetData.GetClass(EResolveClass::Yes))
	{
		return Class->IsChildOf<UTexture2D>();
	}

	return false;
}

TSharedPtr<FMediaImageViewer> FTexture2DImageViewer::FFactory::CreateImageViewer(const FAssetData& InAssetData) const
{
	if (UObject* Object = InAssetData.GetAsset())
	{
		return CreateImageViewer(TNotNull<UObject*>(Object));
	}

	return nullptr;
}

TSharedPtr<FMediaViewerLibraryItem> FTexture2DImageViewer::FFactory::CreateLibraryItem(const FAssetData& InAssetData) const
{
	if (UObject* Object = InAssetData.GetAsset())
	{
		return CreateLibraryItem(TNotNull<UObject*>(Object));
	}

	return nullptr;
}

bool FTexture2DImageViewer::FFactory::SupportsObject(TNotNull<UObject*> InObject) const
{
	return InObject->IsA<UTexture2D>();
}

TSharedPtr<FMediaImageViewer> FTexture2DImageViewer::FFactory::CreateImageViewer(TNotNull<UObject*> InObject) const
{
	if (UTexture2D* Texture = Cast<UTexture2D>(InObject))
	{
		return MakeShared<FTexture2DImageViewer>(Texture);
	}

	return nullptr;
}

TSharedPtr<FMediaViewerLibraryItem> FTexture2DImageViewer::FFactory::CreateLibraryItem(TNotNull<UObject*> InObject) const
{
	if (UTexture2D* Texture = Cast<UTexture2D>(InObject))
	{
		return MakeShared<FTexture2DImageViewer::FItem>(
			FMediaImageViewer::GetObjectDisplayName(Texture),
			FText::Format(LOCTEXT("ToolTipFromat", "{0} [Texture 2D - {1}x{2}]"), FText::FromString(Texture->GetPathName()), FText::AsNumber(Texture->GetSurfaceWidth()), FText::AsNumber(Texture->GetSurfaceHeight())),
			Texture->HasAnyFlags(EObjectFlags::RF_Transient) || Texture->IsIn(GetTransientPackage()),
			Texture
		);
	}

	return nullptr;	
}

bool FTexture2DImageViewer::FFactory::SupportsItemType(FName InItemType) const
{
	return InItemType == FTexture2DImageViewer::ItemTypeName;
}

TSharedPtr<FMediaViewerLibraryItem> FTexture2DImageViewer::FFactory::CreateLibraryItem(const FMediaViewerLibraryItem& InSavedItem) const
{
	FAssetData AssetData;

	if (IAssetRegistry::Get()->TryGetAssetByObjectPath(InSavedItem.GetStringValue(), AssetData) == UE::AssetRegistry::EExists::Exists)
	{
		return MakeShared<FTexture2DImageViewer::FItem>(FPrivateToken(), InSavedItem);
	}

	return nullptr;
}

FTexture2DImageViewer::FItem::FItem(const FText& InName, const FText& InToolTip, bool bInTransient, TNotNull<UTexture2D*> InTexture)
	: FItem(FGuid::NewGuid(), InName, InToolTip, bInTransient, InTexture)
{
}

FTexture2DImageViewer::FItem::FItem(const FGuid& InId, const FText& InName, const FText& InToolTip, bool bInTransient,
	TNotNull<UTexture2D*> InTexture)
	: FMediaViewerLibraryItem(InId, InName, InToolTip, bInTransient, InTexture->GetPathName())
{
}

FTexture2DImageViewer::FItem::FItem(FPrivateToken&& InPrivateToken, const FMediaViewerLibraryItem& InItem)
	: FMediaViewerLibraryItem(InItem.GetId(), InItem.Name, InItem.ToolTip, InItem.IsTransient(), InItem.GetStringValue())
{
}

FName FTexture2DImageViewer::FItem::GetItemType() const
{
	return FTexture2DImageViewer::ItemTypeName;
}

FText FTexture2DImageViewer::FItem::GetItemTypeDisplayName() const
{
	return LOCTEXT("Texture2D", "Texture 2D");
}

FSlateColor FTexture2DImageViewer::FItem::GetItemTypeColor() const
{
	return GetClassColor(UTexture2D::StaticClass());
}

TSharedPtr<FSlateBrush> FTexture2DImageViewer::FItem::CreateThumbnail()
{
	if (StringValue.IsEmpty())
	{
		return nullptr;
	}

	if (!Texture)
	{
		Texture = LoadObject<UTexture2D>(GetTransientPackage(), *StringValue);

		if (!Texture)
		{
			return nullptr;
		}
	}

	TSharedRef<FSlateImageBrush> ThumbnailBrush = MakeShared<FSlateImageBrush>(Texture, FVector2D(Texture->GetSurfaceWidth(), Texture->GetSurfaceDepth()));

	return ThumbnailBrush;
}

TSharedPtr<FMediaImageViewer> FTexture2DImageViewer::FItem::CreateImageViewer() const
{
	if (StringValue.IsEmpty())
	{
		return nullptr;
	}

	if (!Texture)
	{
		Texture = LoadAssetFromString<UTexture2D>(StringValue);

		if (!Texture)
		{
			return nullptr;
		}
	}

	if (Id.IsValid())
	{
		return MakeShared<FTexture2DImageViewer>(Id, Texture);
	}

	return MakeShared<FTexture2DImageViewer>(Texture);
}

TSharedPtr<FMediaViewerLibraryItem> FTexture2DImageViewer::FItem::Clone() const
{
	if (StringValue.IsEmpty())
	{
		return nullptr;
	}

	return MakeShared<FTexture2DImageViewer::FItem>(FPrivateToken(), *this);
}

TOptional<FAssetData> FTexture2DImageViewer::FItem::AsAsset() const
{
	if (::IsValid(Texture))
	{
		if (Texture->IsAsset())
		{
			return FAssetData(Texture);
		}
	}

	return {};
}

FString FTexture2DImageViewer::FItem::GetReferencerName() const
{
	static const FString ReferencerName = TEXT("FTexture2DImageViewer::FItem");
	return ReferencerName;
}

void FTexture2DImageViewer::FItem::AddReferencedObjects(FReferenceCollector& InCollector)
{
	if (Texture)
	{
		InCollector.AddReferencedObject(Texture);
	}
}

FTexture2DImageViewer::FTexture2DImageViewer(TNotNull<UTexture2D*> InTexture)
	: FTexture2DImageViewer(FGuid::NewGuid(), InTexture)
{
}

FTexture2DImageViewer::FTexture2DImageViewer(TNotNull<UTexture2D*> InTexture, const FText& InDisplayName)
	: FTexture2DImageViewer(InTexture)
{
	ImageInfo.DisplayName = InDisplayName;
}

FTexture2DImageViewer::FTexture2DImageViewer(const FGuid& InId, TNotNull<UTexture2D*> InTexture)
	: FMediaImageViewer({
		InId,
		FIntPoint(InTexture->GetSurfaceWidth(), InTexture->GetSurfaceHeight()),
		InTexture->GetPlatformData()->Mips.Num(),
		FMediaImageViewer::GetObjectDisplayName(InTexture)
	})
	, SampleCache(MakeShared<FTextureSampleCache>(InTexture, InTexture->GetPixelFormat()))
	, bValidImageSize(false)
{
	TextureSettings.Texture = InTexture;
	FVector2D ImageSize = FVector2D::ZeroVector;

	if (TOptional<FIntPoint> TextureSize = GetTextureSize())
	{
		ImageSize.X = TextureSize->X;
		ImageSize.Y = TextureSize->Y;
		bValidImageSize = true;
	}

	Brush = MakeShared<FSlateImageBrush>(InTexture, ImageSize);
}

TSharedPtr<FMediaViewerLibraryItem> FTexture2DImageViewer::CreateLibraryItem() const
{
	UTexture2D* Texture = TextureSettings.Texture;

	if (!Texture)
	{
		return nullptr;
	}

	return MakeShared<FItem>(
		ImageInfo.Id,
		FMediaImageViewer::GetObjectDisplayName(Texture),
		FText::Format(LOCTEXT("ToolTipFromat", "{0} [Texture 2D - {1}x{2}]"), FText::FromString(Texture->GetPathName()), FText::AsNumber(Texture->GetSurfaceWidth()), FText::AsNumber(Texture->GetSurfaceHeight())),
		Texture->HasAnyFlags(EObjectFlags::RF_Transient) || Texture->IsIn(GetTransientPackage()),
		Texture
	);
}

TOptional<TVariant<FColor, FLinearColor>> FTexture2DImageViewer::GetPixelColor(const FIntPoint& InPixelCoords, int32 InMipLevel) const
{
	if (!SampleCache.IsValid() || !SampleCache->IsValid())
	{
		return {};
	}

	if (InPixelCoords.X < 0 || InPixelCoords.Y < 0)
	{
		return {};
	}

	if (InPixelCoords.X >= ImageInfo.Size.X || InPixelCoords.Y >= ImageInfo.Size.Y)
	{
		return {};
	}

	FIntPoint MirroredPixelCoords = InPixelCoords;

	if (EnumHasAllFlags(static_cast<EMediaImageMirrorPlane>(PaintSettings.MirrorPlanes), EMediaImageMirrorPlane::Horizontal))
	{
		MirroredPixelCoords.X = ImageInfo.Size.X - InPixelCoords.X - 1;
	}

	if (EnumHasAllFlags(static_cast<EMediaImageMirrorPlane>(PaintSettings.MirrorPlanes), EMediaImageMirrorPlane::Vertical))
	{
		MirroredPixelCoords.Y = ImageInfo.Size.Y - InPixelCoords.Y - 1;
	}

	if (const FLinearColor* PixelColor = SampleCache->GetPixelColor(MirroredPixelCoords))
	{
		TVariant<FColor, FLinearColor> PixelColorVariant;
		PixelColorVariant.Set<FLinearColor>(FMediaViewerUtils::CorrectGamma(*PixelColor));

		return PixelColorVariant;
	}

	return {};
}

TSharedPtr<FStructOnScope> FTexture2DImageViewer::GetCustomSettingsOnScope() const
{
	return MakeShared<FStructOnScope>(
		FTexture2DImageViewerSettings::StaticStruct(),
		reinterpret_cast<uint8*>(&const_cast<FTexture2DImageViewerSettings&>(TextureSettings))
	);
}

void FTexture2DImageViewer::AddReferencedObjects(FReferenceCollector& InCollector)
{
	FMediaImageViewer::AddReferencedObjects(InCollector);

	InCollector.AddPropertyReferencesWithStructARO(
		FTexture2DImageViewerSettings::StaticStruct(),
		&TextureSettings
	);
}

void FTexture2DImageViewer::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged)
{
	FMediaImageViewer::NotifyPostChange(InPropertyChangedEvent, InPropertyThatChanged);

	if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(FMediaImagePaintSettings, MirrorPlanes))
	{
		OnMirrored();
	}
}

void FTexture2DImageViewer::OnMirrored()
{
	if (SampleCache.IsValid())
	{
		SampleCache->Invalidate();
	}
}

TOptional<FIntPoint> FTexture2DImageViewer::GetTextureSize() const
{
	if (!TextureSettings.Texture)
	{
		return {};
	}

	FTexturePlatformData* PlatformData = TextureSettings.Texture->GetPlatformData();

	if (!PlatformData)
	{
		return {};
	}

	if (FTextureResource* TextureResource = TextureSettings.Texture->GetResource())
	{
		if (TextureResource->IsProxy())
		{
			return {};
		}
	}

	return FIntPoint(TextureSettings.Texture->GetSurfaceWidth(), TextureSettings.Texture->GetSurfaceHeight());
}

bool FTexture2DImageViewer::OnOwnerCleanup(UObject* InObject)
{
	if (InObject && TextureSettings.Texture && 
		(TextureSettings.Texture == InObject || TextureSettings.Texture->IsIn(InObject)))
	{
		SampleCache.Reset();

		if (Brush.IsValid())
		{
			Brush->SetResourceObject(nullptr);
		}

		return true;
	}

	return false;
}

void FTexture2DImageViewer::PaintImage(FMediaImageSlatePaintParams& InPaintParams, const FMediaImageSlatePaintGeometry& InPaintGeometry)
{
	if (!bValidImageSize)
	{
		if (TOptional<FIntPoint> TextureSize = GetTextureSize())
		{
			ImageInfo.Size.X = TextureSettings.Texture->GetSurfaceWidth();
			ImageInfo.Size.Y = TextureSettings.Texture->GetSurfaceHeight();
			bValidImageSize = true;

			if (Brush.IsValid())
			{
				Brush->SetImageSize(FVector2D(
					TextureSettings.Texture->GetSurfaceWidth(),
					TextureSettings.Texture->GetSurfaceHeight()
				));
			}
		}
	}

	FMediaImageViewer::PaintImage(InPaintParams, InPaintGeometry);
}

FString FTexture2DImageViewer::GetReferencerName() const
{
	static const FString ReferencerName = TEXT("FTexture2DImageViewer");
	return ReferencerName;
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
