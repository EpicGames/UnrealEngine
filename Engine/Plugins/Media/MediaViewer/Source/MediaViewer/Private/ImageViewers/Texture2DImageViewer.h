// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImageViewer/IMediaImageViewerFactory.h"
#include "ImageViewer/MediaImageViewer.h"
#include "Library/MediaViewerLibraryItem.h"

#include "ImageViewers/TextureSampleCache.h"
#include "Math/MathFwd.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"

#include "Texture2DImageViewer.generated.h"

class UTexture2D;

USTRUCT()
struct FTexture2DImageViewerSettings
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Texture")
	TObjectPtr<UTexture2D> Texture = nullptr;
};

namespace UE::MediaViewer::Private
{

class FTexture2DImageViewer : public FMediaImageViewer
{
public:
	struct FFactory : public IMediaImageViewerFactory
	{
		FFactory()
		{
			Priority = 5000;
		}

		virtual bool SupportsAsset(const FAssetData& InAssetData) const override;
		virtual TSharedPtr<FMediaImageViewer> CreateImageViewer(const FAssetData& InAssetData) const override;
		virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(const FAssetData& InAssetData) const override;

		virtual bool SupportsObject(TNotNull<UObject*> InObject) const override;
		virtual TSharedPtr<FMediaImageViewer> CreateImageViewer(TNotNull<UObject*> InObject) const override;
		virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(TNotNull<UObject*> InObject) const override;

		virtual bool SupportsItemType(FName InItemType) const override;
		virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(const FMediaViewerLibraryItem& InSavedItem) const override;
	};

	struct FItem : public FMediaViewerLibraryItem, public FGCObject
	{
		mutable TObjectPtr<UTexture2D> Texture;

		FItem(const FText& InName, const FText& InToolTip, bool bInTransient, TNotNull<UTexture2D*> InTexture);

		FItem(const FGuid& InId, const FText& InName, const FText& InToolTip, bool bInTransient, TNotNull<UTexture2D*> InTexture);

		FItem(FPrivateToken&& InPrivateToken, const FMediaViewerLibraryItem& InItem);

		virtual ~FItem() override = default;

		//~ Begin FMediaViewerLibraryItem
		virtual FName GetItemType() const override;
		virtual FText GetItemTypeDisplayName() const override;
		virtual FSlateColor GetItemTypeColor() const override;
		virtual TSharedPtr<FSlateBrush> CreateThumbnail() override;
		virtual TSharedPtr<FMediaImageViewer> CreateImageViewer() const override;
		virtual TSharedPtr<FMediaViewerLibraryItem> Clone() const override;
		virtual TOptional<FAssetData> AsAsset() const override;
		//~ End FMediaViewerLibraryItem

		//~ Begin FGCObject
		virtual FString GetReferencerName() const override;
		virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
		//~ End FGCObject
	};

	static const FLazyName ItemTypeName;

	FTexture2DImageViewer(TNotNull<UTexture2D*> InTexture);
	FTexture2DImageViewer(const FGuid& InId, TNotNull<UTexture2D*> InTexture);

	/**
	 * Constructs a viewer for InTexture but overrides the display name shown in the viewer
	 * UI with InDisplayName. Use this when the texture's UObject name (which cannot contain
	 * dots or other characters allowed in file names) does not match the label the user
	 * expects to see.
	 */
	FTexture2DImageViewer(TNotNull<UTexture2D*> InTexture, const FText& InDisplayName);

	virtual ~FTexture2DImageViewer() override = default;

	//~ Begin FMediaImageViewer
	virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem() const override;
	virtual bool OnOwnerCleanup(UObject* InObject) override;
	virtual TOptional<TVariant<FColor, FLinearColor>> GetPixelColor(const FIntPoint& InPixelCoords, int32 InMipLevel) const override;
	virtual TSharedPtr<FStructOnScope> GetCustomSettingsOnScope() const override;
	virtual void OnMirrored() override;
	//~ End FMediaImageViewer

	//~ Begin FGCObject
	virtual FString GetReferencerName() const override;
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	//~ End FGCObject

	//~ Begin FNotifyHook
	virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged) override;
	//~ End FNotifyHook

protected:
	FTexture2DImageViewerSettings TextureSettings;
	TSharedPtr<FTextureSampleCache> SampleCache;
	bool bValidImageSize;

	TOptional<FIntPoint> GetTextureSize() const;

	//~ Begin FMediaImageViewer
	virtual void PaintImage(FMediaImageSlatePaintParams& InPaintParams, const FMediaImageSlatePaintGeometry& InPaintGeometry) override;
	//~ End FMediaImageViewer
};

} // UE::MediaViewer::Private
