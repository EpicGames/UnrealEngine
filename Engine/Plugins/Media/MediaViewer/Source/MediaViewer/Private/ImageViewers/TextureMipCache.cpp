// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageViewers/TextureMipCache.h"

#include "Engine/Texture.h"

namespace UE::MediaViewer::Private
{

FTextureMipCache::FTextureMipCache()
{
	Invalidate();
}

FTextureMipCache::FTextureMipCache(TNotNull<UTexture*> InTexture)
{
	if (!InTexture->Source.IsValid())
	{
		Invalidate();
		return;
	}

	TextureWeak = InTexture;
	Mips.SetNumZeroed(InTexture->Source.GetNumMips());
}

const FImage* FTextureMipCache::GetMipImage(int32 InMipLevel) const
{
	if (!Mips.IsValidIndex(InMipLevel))
	{
		return nullptr;
	}

	if (Mips[InMipLevel].State == EMipState::NotCached)
	{
		const_cast<FTextureMipCache*>(this)->CreateMipImage(InMipLevel);
	}

	if (Mips[InMipLevel].State == EMipState::Failed)
	{
		return nullptr;
	}

	return &Mips[InMipLevel].Image;
}

void FTextureMipCache::Invalidate()
{
	UTexture* Texture = TextureWeak.Get();

	if (!Texture)
	{
		Mips.SetNum(0);
		return;
	}

	Mips.SetNumZeroed(Texture->Source.GetNumMips());
}

void FTextureMipCache::CreateMipImage(int32 InMipLevel)
{
	UTexture* Texture = TextureWeak.Get();

	if (!Texture)
	{
		Mips[InMipLevel].State = EMipState::Failed;
		return;
	}

	Mips[InMipLevel].State = Texture->Source.GetMipImage(Mips[InMipLevel].Image, InMipLevel)
		? EMipState::Cached
		: EMipState::Failed;	
}

} // UE::MediaViewer::Private
