// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "ImageCore.h"
#include "Misc/NotNull.h"
#include "PixelFormat.h"
#include "UObject/WeakObjectPtr.h"

class UTexture;

namespace UE::MediaViewer::Private
{

class FTextureMipCache
{
public:
	enum class EMipState
	{
		NotCached = 0,
		Cached,
		Failed
	};

	struct FMipCache
	{
		EMipState State;
		FImage Image;
	};

	FTextureMipCache();
	FTextureMipCache(TNotNull<UTexture*> InTexture);

	const FImage* GetMipImage(int32 InMipLevel) const;

	void Invalidate();
	
protected:
	TWeakObjectPtr<UTexture> TextureWeak;
	TArray<FMipCache> Mips;

	void CreateMipImage(int32 InMipLevel);
};

} // UE::MediaViewer::Private
