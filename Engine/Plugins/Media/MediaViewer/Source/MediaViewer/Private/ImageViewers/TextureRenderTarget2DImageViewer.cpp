// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageViewers/TextureRenderTarget2DImageViewer.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "Brushes/SlateImageBrush.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Math/Vector2D.h"
#include "MediaViewerUtils.h"
#include "Misc/Optional.h"
#include "Misc/TVariant.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureRenderTarget2DImageViewer)

#define LOCTEXT_NAMESPACE "TextureRenderTarget2DImageViewer"

namespace UE::MediaViewer::Private
{

const FLazyName FTextureRenderTarget2DImageViewer::ItemTypeName = TEXT("TextureRenderTarget2D");

bool FTextureRenderTarget2DImageViewer::FFactory::SupportsAsset(const FAssetData& InAssetData) const
{
	if (UClass* Class = InAssetData.GetClass(EResolveClass::Yes))
	{
		return Class->IsChildOf<UTextureRenderTarget2D>();
	}

	return false;
}

TSharedPtr<FMediaImageViewer> FTextureRenderTarget2DImageViewer::FFactory::CreateImageViewer(const FAssetData& InAssetData) const
{
	if (UObject* Object = InAssetData.GetAsset())
	{
		return CreateImageViewer(TNotNull<UObject*>(Object));
	}

	return nullptr;
}

TSharedPtr<FMediaViewerLibraryItem> FTextureRenderTarget2DImageViewer::FFactory::CreateLibraryItem(const FAssetData& InAssetData) const
{
	if (UObject* Object = InAssetData.GetAsset())
	{
		return CreateLibraryItem(TNotNull<UObject*>(Object));
	}

	return nullptr;
}

bool FTextureRenderTarget2DImageViewer::FFactory::SupportsObject(TNotNull<UObject*> InObject) const
{
	return InObject->IsA<UTextureRenderTarget2D>();
}

TSharedPtr<FMediaImageViewer> FTextureRenderTarget2DImageViewer::FFactory::CreateImageViewer(TNotNull<UObject*> InObject) const
{
	if (UTextureRenderTarget2D* RenderTarget = Cast<UTextureRenderTarget2D>(InObject))
	{
		return MakeShared<FTextureRenderTarget2DImageViewer>(RenderTarget);
	}

	return nullptr;
}

TSharedPtr<FMediaViewerLibraryItem> FTextureRenderTarget2DImageViewer::FFactory::CreateLibraryItem(TNotNull<UObject*> InObject) const
{
	if (UTextureRenderTarget2D* RenderTarget = Cast<UTextureRenderTarget2D>(InObject))
	{
		return MakeShared<FTextureRenderTarget2DImageViewer::FItem>(
			FMediaImageViewer::GetObjectDisplayName(RenderTarget),
			FText::Format(LOCTEXT("ToolTipFromat", "{0} [Render Target 2D - {1}x{2}]"), FText::FromString(RenderTarget->GetPathName()), FText::AsNumber(RenderTarget->GetSurfaceWidth()), FText::AsNumber(RenderTarget->GetSurfaceHeight())),
			RenderTarget->HasAnyFlags(EObjectFlags::RF_Transient) || RenderTarget->IsIn(GetTransientPackage()),
			RenderTarget
		);
	}

	return nullptr;	
}

bool FTextureRenderTarget2DImageViewer::FFactory::SupportsItemType(FName InItemType) const
{
	return InItemType == FTextureRenderTarget2DImageViewer::ItemTypeName;
}

TSharedPtr<FMediaViewerLibraryItem> FTextureRenderTarget2DImageViewer::FFactory::CreateLibraryItem(const FMediaViewerLibraryItem& InSavedItem) const
{
	// While we support in-memory render targets, anything "saved" must be an asset.
	FAssetData AssetData;

	if (IAssetRegistry::Get()->TryGetAssetByObjectPath(InSavedItem.GetStringValue(), AssetData) == UE::AssetRegistry::EExists::Exists)
	{
		return MakeShared<FTextureRenderTarget2DImageViewer::FItem>(FPrivateToken(), InSavedItem);
	}

	return nullptr;
}

FTextureRenderTarget2DImageViewer::FItem::FItem(const FText& InName, const FText& InToolTip, bool bInTransient, TNotNull<UTextureRenderTarget2D*> InRenderTarget)
	: FItem(FGuid::NewGuid(), InName, InToolTip, bInTransient, InRenderTarget)
{
}

FTextureRenderTarget2DImageViewer::FItem::FItem(const FGuid& InId, const FText& InName, const FText& InToolTip, bool bInTransient,
	TNotNull<UTextureRenderTarget2D*> InRenderTarget)
	: FMediaViewerLibraryItem(InId, InName, InToolTip, bInTransient, InRenderTarget->GetPathName())
{
}

FTextureRenderTarget2DImageViewer::FItem::FItem(FPrivateToken&& InPrivateToken, const FMediaViewerLibraryItem& InItem)
	: FMediaViewerLibraryItem(InItem.GetId(), InItem.Name, InItem.ToolTip, InItem.IsTransient(), InItem.GetStringValue())
{
}

FName FTextureRenderTarget2DImageViewer::FItem::GetItemType() const
{
	return FTextureRenderTarget2DImageViewer::ItemTypeName;
}

FText FTextureRenderTarget2DImageViewer::FItem::GetItemTypeDisplayName() const
{
	return LOCTEXT("TextureRenderTarget2D", "Render Target 2D");
}

FSlateColor FTextureRenderTarget2DImageViewer::FItem::GetItemTypeColor() const
{
	return GetClassColor(UTextureRenderTarget2D::StaticClass());
}

TSharedPtr<FSlateBrush> FTextureRenderTarget2DImageViewer::FItem::CreateThumbnail()
{
	UTextureRenderTarget2D* RenderTarget = LoadAssetFromString<UTextureRenderTarget2D>(StringValue);

	if (!RenderTarget)
	{
		return nullptr;
	}

	TSharedRef<FSlateImageBrush> ThumbnailBrush = MakeShared<FSlateImageBrush>(RenderTarget, FVector2D(RenderTarget->GetSurfaceWidth(), RenderTarget->GetSurfaceDepth()));

	return ThumbnailBrush;
}

TSharedPtr<FMediaImageViewer> FTextureRenderTarget2DImageViewer::FItem::CreateImageViewer() const
{
	UTextureRenderTarget2D* RenderTarget = LoadAssetFromString<UTextureRenderTarget2D>(StringValue);

	if (!RenderTarget)
	{
		return nullptr;
	}

	if (Id.IsValid())
	{
		return MakeShared<FTextureRenderTarget2DImageViewer>(Id, RenderTarget);
	}

	return MakeShared<FTextureRenderTarget2DImageViewer>(RenderTarget);
}

TSharedPtr<FMediaViewerLibraryItem> FTextureRenderTarget2DImageViewer::FItem::Clone() const
{
	if (StringValue.IsEmpty())
	{
		return nullptr;
	}

	return MakeShared<FTextureRenderTarget2DImageViewer::FItem>(FPrivateToken(), *this);
}

TOptional<FAssetData> FTextureRenderTarget2DImageViewer::FItem::AsAsset() const
{
	if (UTextureRenderTarget2D* RenderTarget = LoadAssetFromString<UTextureRenderTarget2D>(StringValue))
	{
		if (RenderTarget->IsAsset())
		{
			return FAssetData(RenderTarget);
		}
	}

	return {};
}

FTextureRenderTarget2DImageViewer::FTextureRenderTarget2DImageViewer(TNotNull<UTextureRenderTarget2D*> InRenderTarget)
	: FTextureRenderTarget2DImageViewer(FGuid::NewGuid(), InRenderTarget)
{
}

FTextureRenderTarget2DImageViewer::FTextureRenderTarget2DImageViewer(const FGuid& InId, TNotNull<UTextureRenderTarget2D*> InRenderTarget)
	: FMediaImageViewer({
		InId,
		FIntPoint(InRenderTarget->GetSurfaceWidth(), InRenderTarget->GetSurfaceHeight()),
		InRenderTarget->GetNumMips(),
		FMediaImageViewer::GetObjectDisplayName(InRenderTarget)
	})
{
	DrawEffects |= ESlateDrawEffect::PreMultipliedAlpha;

	RenderTargetSettings.RenderTarget = InRenderTarget;

	Brush = MakeShared<FSlateImageBrush>(
		InRenderTarget,
		FVector2D(InRenderTarget->GetSurfaceWidth(), InRenderTarget->GetSurfaceHeight())
	);
	
	SampleCache = MakeShared<FTextureSampleCache>(InRenderTarget, InRenderTarget->GetFormat());
}

TSharedPtr<FMediaViewerLibraryItem> FTextureRenderTarget2DImageViewer::CreateLibraryItem() const
{
	UTextureRenderTarget2D* RenderTarget = RenderTargetSettings.RenderTarget;

	if (!RenderTarget)
	{
		return nullptr;
	}

	return MakeShared<FItem>(
		ImageInfo.Id,
		FMediaImageViewer::GetObjectDisplayName(RenderTarget),
		FText::Format(LOCTEXT("ToolTipFromat", "{0} [Render Target 2D - {1}x{2}]"), FText::FromString(RenderTarget->GetPathName()), FText::AsNumber(RenderTarget->GetSurfaceWidth()), FText::AsNumber(RenderTarget->GetSurfaceHeight())),
		RenderTarget->HasAnyFlags(EObjectFlags::RF_Transient) || RenderTarget->IsIn(GetTransientPackage()),
		RenderTarget
	);
}

TOptional<TVariant<FColor, FLinearColor>> FTextureRenderTarget2DImageViewer::GetPixelColor(const FIntPoint& InPixelCoords, int32 InMipLevel) const
{
	if (!SampleCache.IsValid() || !SampleCache->IsValid())
	{
		return {};
	}

	// Always mark the sample cache dirty. We have no control over the state of the media texture.
	SampleCache->MarkDirty();

	if (InPixelCoords.X < 0 || InPixelCoords.Y < 0)
	{
		SampleCache->Invalidate();
		return {};
	}

	if (InPixelCoords.X >= ImageInfo.Size.X || InPixelCoords.Y >= ImageInfo.Size.Y)
	{
		SampleCache->Invalidate();
		return {};
	}

	FIntPoint MirroredPixelCoords = InPixelCoords;

	if (EnumHasAllFlags(static_cast<EMediaImageMirrorPlane>(PaintSettings.MirrorPlanes), EMediaImageMirrorPlane::Horizontal))
	{
		MirroredPixelCoords.X = ImageInfo.Size.X - InPixelCoords.X - 1;
	}

	if (EnumHasAllFlags(static_cast<EMediaImageMirrorPlane>(PaintSettings.MirrorPlanes), EMediaImageMirrorPlane::Vertical))
	{
		MirroredPixelCoords.Y = ImageInfo.Size.Y - InPixelCoords.Y - 1;
	}

	if (const FLinearColor* PixelColor = SampleCache->GetPixelColor(MirroredPixelCoords))
	{
		TVariant<FColor, FLinearColor> PixelColorVariant;
		PixelColorVariant.Set<FLinearColor>(FMediaViewerUtils::CorrectGamma(*PixelColor));

		return PixelColorVariant;
	}

	return {};
}

TSharedPtr<FStructOnScope> FTextureRenderTarget2DImageViewer::GetCustomSettingsOnScope() const
{
	return MakeShared<FStructOnScope>(
		FTextureRenderTarget2DImageViewerSettings::StaticStruct(),
		reinterpret_cast<uint8*>(&const_cast<FTextureRenderTarget2DImageViewerSettings&>(RenderTargetSettings))
	);
}

void FTextureRenderTarget2DImageViewer::AddReferencedObjects(FReferenceCollector& InCollector)
{
	FMediaImageViewer::AddReferencedObjects(InCollector);

	InCollector.AddPropertyReferencesWithStructARO(
		FTextureRenderTarget2DImageViewerSettings::StaticStruct(),
		&RenderTargetSettings
	);
}

void FTextureRenderTarget2DImageViewer::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged)
{
	FMediaImageViewer::NotifyPostChange(InPropertyChangedEvent, InPropertyThatChanged);

	if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(FMediaImagePaintSettings, MirrorPlanes))
	{
		OnMirrored();
	}
}

void FTextureRenderTarget2DImageViewer::OnMirrored()
{
	if (SampleCache.IsValid())
	{
		SampleCache->Invalidate();
	}
}

bool FTextureRenderTarget2DImageViewer::OnOwnerCleanup(UObject* InObject)
{
	if (InObject && RenderTargetSettings.RenderTarget && 
		(RenderTargetSettings.RenderTarget == InObject || RenderTargetSettings.RenderTarget->IsIn(InObject)))
	{
		SampleCache.Reset();

		if (Brush.IsValid())
		{
			Brush->SetResourceObject(nullptr);
		}

		return true;
	}

	return false;
}

FString FTextureRenderTarget2DImageViewer::GetReferencerName() const
{
	static const FString ReferencerName = TEXT("FTextureRenderTarget2DImageViewer");
	return ReferencerName;
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
