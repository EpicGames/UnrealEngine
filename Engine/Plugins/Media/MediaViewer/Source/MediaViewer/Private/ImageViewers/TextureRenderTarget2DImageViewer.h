// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImageViewer/IMediaImageViewerFactory.h"
#include "ImageViewer/MediaImageViewer.h"
#include "Library/MediaViewerLibraryItem.h"

#include "ImageViewers/TextureSampleCache.h"
#include "Templates/SharedPointer.h"

#include "TextureRenderTarget2DImageViewer.generated.h"

class UTextureRenderTarget2D;

USTRUCT()
struct FTextureRenderTarget2DImageViewerSettings
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Texture")
	TObjectPtr<UTextureRenderTarget2D> RenderTarget = nullptr;
};

namespace UE::MediaViewer::Private
{

class FTextureRenderTarget2DImageViewer : public FMediaImageViewer
{
public:
	struct FFactory : public IMediaImageViewerFactory
	{
		FFactory()
		{
			Priority = 4000;
		}

		virtual bool SupportsAsset(const FAssetData& InAssetData) const override;
		virtual TSharedPtr<FMediaImageViewer> CreateImageViewer(const FAssetData& InAssetData) const override;
		virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(const FAssetData& InAssetData) const override;

		virtual bool SupportsObject(TNotNull<UObject*> InObject) const override;
		virtual TSharedPtr<FMediaImageViewer> CreateImageViewer(TNotNull<UObject*> InObject) const override;
		virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(TNotNull<UObject*> InObject) const override;

		virtual bool SupportsItemType(FName InItemType) const override;
		virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(const FMediaViewerLibraryItem& InSavedItem) const override;
	};

	struct FItem : public FMediaViewerLibraryItem
	{
		FItem(const FText& InName, const FText& InToolTip, bool bInTransient, TNotNull<UTextureRenderTarget2D*> InRenderTarget);

		FItem(const FGuid& InId, const FText& InName, const FText& InToolTip, bool bInTransient, TNotNull<UTextureRenderTarget2D*> InRenderTarget);

		FItem(FPrivateToken&& InPrivateToken, const FMediaViewerLibraryItem& InItem);

		virtual ~FItem() override = default;

		//~ Begin FMediaViewerLibraryItem
		virtual FName GetItemType() const override;
		virtual FText GetItemTypeDisplayName() const override;
		virtual FSlateColor GetItemTypeColor() const override;
		virtual TSharedPtr<FSlateBrush> CreateThumbnail() override;
		virtual TSharedPtr<FMediaImageViewer> CreateImageViewer() const override;
		virtual TSharedPtr<FMediaViewerLibraryItem> Clone() const override;
		virtual TOptional<FAssetData> AsAsset() const override;
		//~ End FMediaViewerLibraryItem
	};

	static const FLazyName ItemTypeName;

	FTextureRenderTarget2DImageViewer(TNotNull<UTextureRenderTarget2D*> InTextureRenderTarget2D);
	FTextureRenderTarget2DImageViewer(const FGuid& InId, TNotNull<UTextureRenderTarget2D*> InTextureRenderTarget2D);

	virtual ~FTextureRenderTarget2DImageViewer() = default;

	//~ Begin FMediaImageViewer
	virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem() const override;
	virtual bool OnOwnerCleanup(UObject* InObject) override;
	virtual TOptional<TVariant<FColor, FLinearColor>> GetPixelColor(const FIntPoint& InPixelCoords, int32 InMipLevel) const override;
	virtual TSharedPtr<FStructOnScope> GetCustomSettingsOnScope() const override;
	virtual void OnMirrored() override;
	//~ End FMediaImageViewer

	//~ Begin FGCObject
	virtual FString GetReferencerName() const override;
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	//~ End FGCObject

	//~ Begin FNotifyHook
	virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged) override;
	//~ End FNotifyHook

protected:
	FTextureRenderTarget2DImageViewerSettings RenderTargetSettings;
	TSharedPtr<FTextureSampleCache> SampleCache;
};

} // UE::MediaViewer::Private
