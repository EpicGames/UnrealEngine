// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageViewers/TextureSampleCache.h"

#include "ColorManagement/TransferFunctions.h"
#include "Engine/Texture.h"
#include "Engine/TextureRenderTarget2D.h"
#include "GlobalShader.h"
#include "HAL/CriticalSection.h"
#include "HAL/IConsoleManager.h"
#include "Math/Float16Color.h"
#include "MediaViewerUtils.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeLock.h"
#include "PixelFormat.h"
#include "RHIGPUReadback.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RenderingThread.h"
#include "Templates/UniquePtr.h"
#include "TextureResource.h"
#include "UObject/WeakObjectPtr.h"

#include <atomic>

static TAutoConsoleVariable<bool> CVarMediaViewerEnableColorPicker(
	TEXT("MediaViewer.EnableColorPicker"),
	true,
	TEXT("If true (default), enables async per-pixel readback for the Media Viewer color picker."),
	ECVF_Default
);

namespace UE::MediaViewer::Private
{
/** Intermediate Render Target Size */
constexpr int32 SampleRTSize = 1;

/** Intermediate Render Target Format */
constexpr ETextureRenderTargetFormat SampleRTFormat = RTF_RGBA16f;

class FMediaViewerTextureSampleResource
{
public:
	struct FPixelColorSample
	{
		FIntPoint Coordinates;
		uint8 MipLevel = 0;
		TOptional<FTimespan> Time;
		FLinearColor Color;
	};

	// === Game-thread API ===

	bool ShouldSample(const FIntPoint& InCoordinates, uint8 InMipLevel, const TOptional<FTimespan>& InTime) const
	{
		FScopeLock Lock(&SampleCS);
		return bDirty || !CachedSample.IsSet() || InCoordinates != CachedSample->Coordinates
			|| InMipLevel != CachedSample->MipLevel
			|| (InTime.IsSet() && (CachedSample->Time != InTime.GetValue()));
	}

	const FLinearColor* TryGetCachedColor() const
	{
		FScopeLock Lock(&SampleCS);
		return CachedSample.IsSet() ? &CachedSample->Color : nullptr;
	}

	void MarkDirty()
	{
		FScopeLock Lock(&SampleCS);
		bDirty = true;
	}

	void Invalidate()
	{
		FScopeLock Lock(&SampleCS);
		CachedSample.Reset();
		// Force ShouldSample() to return true once the caller resamples, even if
		// a now-cancelled in-flight readback would have published the same coords.
		bDirty = true;
	}

	// Reserve the single outstanding-pump slot. Returns true if no pump is currently
	// queued and the caller may proceed to ENQUEUE_RENDER_COMMAND. Returns false if a
	// pump is already in flight - the caller should drop this paint's request rather
	// than spam redundant render commands while a readback is pending. The slot is
	// released by ReleasePumpReservation_RenderThread at the lambda's exit.
	bool TryReservePump()
	{
		return !bPumpQueued.exchange(true, std::memory_order_acq_rel);
	}

	// === Render-thread API ===

	void Init_RenderThread()
	{
		check(IsInRenderingThread());
		Readback = MakeUnique<FRHIGPUTextureReadback>(TEXT("MediaViewerColorPickerReadback"));
	}

	void ReleasePumpReservation_RenderThread()
	{
		check(IsInRenderingThread());
		bPumpQueued.store(false, std::memory_order_release);
	}

	void Pump_RenderThread(FRHICommandListImmediate& InRHICmdList, UTexture* InSource, UTextureRenderTarget2D* InIntermediate, const FIntPoint& InCoordinates, uint8 InMipLevel, const TOptional<FTimespan>& InTime)
	{
		check(IsInRenderingThread());

		// Using this to detect media texture.
		const bool bIsMediaTexture = InSource->HasRuntimeVariableSRGB();

		// Drain any in-flight readback that the GPU has finished. The readback
		// object's 1x1 staging texture and fence are kept alive across samples -
		// EnqueueCopy reuses them on the next kick.
		if (bReadbackInFlight && Readback->IsReady())
		{
			int32 RowPitchInPixels = 0;
			if (void* Data = Readback->Lock(RowPitchInPixels))
			{
				const FFloat16Color* Pixel = static_cast<const FFloat16Color*>(Data);
				const FLinearColor RawColor(*Pixel);

				// Media Texture seems to have been converted to linear one time too many.
				const FLinearColor Color = bIsMediaTexture ? UE::Color::Encode(UE::Color::EEncoding::sRGB, RawColor) : RawColor;

				{
					FScopeLock Lock(&SampleCS);
					CachedSample = { PendingCoordinates, PendingMipLevel, PendingTime, Color };
					bDirty = false;
				}

				Readback->Unlock();
			}

			bReadbackInFlight = false;
		}

		// If a readback is still in flight, do not start another one. The next pump will retry.
		if (bReadbackInFlight)
		{
			return;
		}

		FTextureResource* SourceResource = InSource->GetResource();

		if (!SourceResource || !SourceResource->TextureRHI.IsValid())
		{
			return;
		}

		constexpr uint32 MinDimension = 3;

		if (SourceResource->GetSizeX() < MinDimension || SourceResource->GetSizeY() < MinDimension)
		{
			return;
		}

		FTextureResource* IntermediateResource = InIntermediate->GetResource();

		if (!IntermediateResource || !IntermediateResource->TextureRHI.IsValid())
		{
			return;
		}

		// Clamp the requested mip to what the RHI texture actually has. Sources backed
		// by a visibility provider may stream mips in and out, so the viewer's reported
		// mip can briefly exceed NumMips during a transition.
		const uint32 NumMips = FMath::Max<uint32>(1, SourceResource->TextureRHI->GetNumMips());
		const uint8 ClampedMipLevel = static_cast<uint8>(FMath::Min<uint32>(InMipLevel, NumMips - 1));

		// Caller supplies mip 0 coordinates (status bar bounds-checks against
		// ImageInfo.Size, which is mip 0). Clamp to mip 0 first so the cached
		// sample key stays in mip 0 space, then shift down to the target mip.
		const FIntPoint Mip0Coords(
			FMath::Clamp(InCoordinates.X, 0, static_cast<int32>(SourceResource->GetSizeX()) - 1),
			FMath::Clamp(InCoordinates.Y, 0, static_cast<int32>(SourceResource->GetSizeY()) - 1)
		);

		const FIntPoint MipCoords(
			Mip0Coords.X >> ClampedMipLevel,
			Mip0Coords.Y >> ClampedMipLevel
		);

		// 1x1 sub-region blit: source[ClampedMipLevel][ClampedCoords] -> intermediate[0,0].
		{
			FRDGBuilder GraphBuilder(InRHICmdList);

			FRDGTexture* InputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SourceResource->TextureRHI, TEXT("MediaViewerSampleSource")));
			FRDGTexture* OutputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(IntermediateResource->TextureRHI, TEXT("MediaViewerSampleIntermediate")));

			FRDGDrawTextureInfo DrawInfo;
			DrawInfo.Size = FIntPoint(1, 1);
			DrawInfo.SourcePosition = MipCoords;
			DrawInfo.DestPosition = FIntPoint(0, 0);
			DrawInfo.SourceMipIndex = ClampedMipLevel;

			const FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
			AddDrawTexturePass(GraphBuilder, GlobalShaderMap, InputTexture, OutputTexture, DrawInfo);

			GraphBuilder.Execute();
		}

		Readback->EnqueueCopy(
			InRHICmdList,
			IntermediateResource->TextureRHI,
			FIntVector(0, 0, 0),
			/* SourceSlice */ 0,
			FIntVector(1, 1, 1)
		);

		bReadbackInFlight = true;
		// PendingCoordinates is stored in mip 0 space (Mip0Coords) so the next
		// ShouldSample() can compare against the GT-side coords without re-shifting.
		PendingCoordinates = Mip0Coords;
		PendingMipLevel = ClampedMipLevel;
		PendingTime = InTime;
	}

	void CancelInFlight_RenderThread()
	{
		check(IsInRenderingThread());
		// Drop the tracking flag. Any GPU work already submitted will complete and
		// write into the staging texture, but the next EnqueueCopy clears the fence
		// and reuses the staging texture, so the orphaned write is harmless.
		bReadbackInFlight = false;
		PendingCoordinates = FIntPoint::ZeroValue;
		PendingMipLevel = 0;
		PendingTime.Reset();

		// A pump enqueued ahead of the cancel may have already drained an obsolete
		// readback and republished a "ghost" sample after the caller's Invalidate.
		// Wipe the cache so the post-cancel state matches the caller's intent.
		FScopeLock Lock(&SampleCS);
		CachedSample.Reset();
		bDirty = true;
	}

private:
	// Render-thread-only readback state.
	TUniquePtr<FRHIGPUTextureReadback> Readback;
	bool bReadbackInFlight = false;
	FIntPoint PendingCoordinates = FIntPoint::ZeroValue;
	uint8 PendingMipLevel = 0;
	TOptional<FTimespan> PendingTime;

	// Cross-thread enqueue gate. Set by GT in TryReservePump before each
	// ENQUEUE_RENDER_COMMAND, cleared by RT at lambda exit. Prevents a Slate
	// repaint storm from spamming the RT command queue while a previous pump
	// is still pending.
	std::atomic<bool> bPumpQueued{false};

	// Cross-thread cached sample, written by Pump_RenderThread, read by the game thread.
	mutable FCriticalSection SampleCS;
	bool bDirty = true;
	TOptional<FPixelColorSample> CachedSample;
};

FTextureSampleCache::FTextureSampleCache(TNotNull<UTexture*> InTexture, EPixelFormat InPixelFormat)
	: TextureWeak(InTexture)
	, RenderTarget(nullptr)
	, SampleResource(MakeShared<FMediaViewerTextureSampleResource>())
{
	check(IsInGameThread());

	// Construction of FRHIGPUTextureReadback creates an RHI fence and must run on the
	// render thread. FIFO ordering guarantees this Init runs before any subsequent
	// Pump enqueued by GetPixelColor.
	ENQUEUE_RENDER_COMMAND(MediaViewerInitSampleResource)(
		[Resource = SampleResource](FRHICommandListImmediate&)
		{
			Resource->Init_RenderThread();
		}
	);
}

FTextureSampleCache::~FTextureSampleCache()
{
	check(IsInGameThread());

	if (SampleResource)
	{
		// Hand off destruction of the resource (and its readback fence/staging texture)
		// to the render thread. FIFO ordering ensures any in-flight pump command runs
		// to completion before the resource is destroyed.
		ENQUEUE_RENDER_COMMAND(MediaViewerDeleteSampleResource)(
			[Resource = MoveTemp(SampleResource)](FRHICommandListImmediate&) mutable
			{
				Resource.Reset();
			}
		);
	}
}

bool FTextureSampleCache::IsValid() const
{
	return TextureWeak.IsValid();
}

const FLinearColor* FTextureSampleCache::GetPixelColor(const FIntPoint& InPixelCoordinates, uint8 InMipLevel, const TOptional<FTimespan>& InTime)
{
	if (!CVarMediaViewerEnableColorPicker.GetValueOnAnyThread())
	{
		return nullptr;
	}

	UTexture* Texture = TextureWeak.Get();

	if (!Texture || !SampleResource)
	{
		return nullptr;
	}

	// Note: small-source check (MinDimension) lives on the render thread side -
	// UTexture::GetSurfaceWidth/Height returns 0 for not-yet-initialized media textures
	// while the render-side FTextureResource is already valid, so a GT-side check would
	// drop legitimate first-frame samples.
	if (SampleResource->ShouldSample(InPixelCoordinates, InMipLevel, InTime))
	{
		// Lazy-create the 1x1 intermediate render target. UObject construction must happen on the game thread.
		if (!RenderTarget)
		{
			RenderTarget = FMediaViewerUtils::CreateRenderTarget(
				FIntPoint(SampleRTSize, SampleRTSize),
				/* Transparent */ true,
				SampleRTFormat
			);
		}

		// Gate the enqueue on a single outstanding pump: a fast-moving cursor
		// produces a Slate repaint per frame, but each empty pump (with a readback
		// already in flight) is wasted RT work. Skipping the enqueue when one is
		// already pending defeats the spam without sacrificing latency - the next
		// paint after the lambda releases the slot will pick up the latest coords.
		if (RenderTarget && SampleResource->TryReservePump())
		{
			ENQUEUE_RENDER_COMMAND(MediaViewerSamplePixel)
			(
				[ResourceWeak = SampleResource.ToWeakPtr(),
				 SourceWeak = TWeakObjectPtr<UTexture>(Texture),
				 IntermediateWeak = TWeakObjectPtr<UTextureRenderTarget2D>(RenderTarget),
				 InPixelCoordinates, InMipLevel, InTime]
				(FRHICommandListImmediate& RHICmdList)
				{
					TSharedPtr<FMediaViewerTextureSampleResource> Resource = ResourceWeak.Pin();

					if (!Resource.IsValid())
					{
						return;
					}

					ON_SCOPE_EXIT { Resource->ReleasePumpReservation_RenderThread(); };

					UTexture* Source = SourceWeak.Get();
					UTextureRenderTarget2D* Intermediate = IntermediateWeak.Get();

					if (!Source || !Intermediate)
					{
						return;
					}

					Resource->Pump_RenderThread(RHICmdList, Source, Intermediate, InPixelCoordinates, InMipLevel, InTime);
				}
			);
		}
	}

	return SampleResource->TryGetCachedColor();
}

void FTextureSampleCache::MarkDirty()
{
	if (SampleResource)
	{
		SampleResource->MarkDirty();
	}
}

void FTextureSampleCache::Invalidate()
{
	if (!SampleResource)
	{
		return;
	}

	SampleResource->Invalidate();

	// Cancel any in-flight readback so its result doesn't repopulate the cached
	// sample after the caller deliberately cleared it.
	ENQUEUE_RENDER_COMMAND(MediaViewerCancelReadback)(
		[ResourceWeak = SampleResource.ToWeakPtr()](FRHICommandListImmediate&)
		{
			if (const TSharedPtr<FMediaViewerTextureSampleResource> Resource = ResourceWeak.Pin())
			{
				Resource->CancelInFlight_RenderThread();
			}
		}
	);
}

FString FTextureSampleCache::GetReferencerName() const
{
	static const FString ReferencerName = TEXT("FTextureSampleCache");
	return ReferencerName;
}

void FTextureSampleCache::AddReferencedObjects(FReferenceCollector& InCollector)
{
	if (RenderTarget)
	{
		InCollector.AddReferencedObject(RenderTarget);
	}
}

} // UE::MediaViewer::Private
