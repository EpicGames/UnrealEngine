// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Color.h"
#include "Math/IntPoint.h"
#include "Misc/NotNull.h"
#include "Misc/Optional.h"
#include "Misc/Timespan.h"
#include "Templates/SharedPointer.h"
#include "UObject/GCObject.h"
#include "UObject/WeakObjectPtr.h"

class UTexture;
class UTextureRenderTarget2D;
enum EPixelFormat : uint8;

namespace UE::MediaViewer::Private
{

class FMediaViewerTextureSampleResource;

/**
 * Asynchronous single-pixel color sampler for an arbitrary UTexture.
 *
 * Used by the Media Viewer color picker / status bar. A 1x1 sub-region of the source
 * is blitted into a tiny RTF_RGBA16f intermediate and read back via
 * FRHIGPUTextureReadback, so sampling is non-blocking even on multi-gigabyte sources.
 * Game-thread callers get the most recently completed sample; results lag by ~1-2
 * frames during rapid movement.
 *
 * Gated by CVar MediaViewer.EnableColorPicker (default true). All public methods are
 * game-thread only; cross-thread state lives in FMediaViewerTextureSampleResource.
 * Must be owned via TSharedPtr.
 */
class FTextureSampleCache : public TSharedFromThis<FTextureSampleCache>, public FGCObject
{
public:
	UE_NONCOPYABLE(FTextureSampleCache);

	/**
	 * Constructs a cache that samples colors from the given texture.
	 * @param InTexture     Source texture; held weakly, sampling no-ops once it's GC'd.
	 * @param InPixelFormat Reserved; currently unused (path normalizes through RTF_RGBA16f).
	 */
	FTextureSampleCache(TNotNull<UTexture*> InTexture, EPixelFormat InPixelFormat);

	/** Tears down the readback resource on the render thread; any in-flight result is discarded. */
	virtual ~FTextureSampleCache() override;

	/** Returns true if the source texture is still alive. */
	bool IsValid() const;

	/**
	 * Returns the last sampled color, kicking a new readback if coordinates,
	 * mip level, or media time differ from the cached sample. Non-blocking.
	 * Out-of-bounds coordinates are clamped on the render thread.
	 *
	 * @param InPixelCoordinates Coordinates in mip 0 space. Internally shifted
	 *                           to the target mip's dimensions before sampling.
	 * @param InMipLevel         Mip to sample. Clamped to the source's mip range;
	 *                           the picker tracks the viewer's displayed mip so
	 *                           visibility-provider sources sample fresh data.
	 * @param InTime             Optional media time; forces a new sample on frame
	 *                           changes at the same pixel for time-varying sources.
	 * @return Pointer to the last published color, or nullptr if no sample has
	 *         completed yet. Valid until the next call on this cache.
	 */
	const FLinearColor* GetPixelColor(const FIntPoint& InPixelCoordinates, uint8 InMipLevel = 0, const TOptional<FTimespan>& InTime = {});

	/** Forces the next GetPixelColor to kick a fresh readback even if coords/time match. */
	void MarkDirty();

	/**
	 * Clears the cached sample and discards the result of any in-flight readback
	 * so it cannot republish a stale color. The submitted GPU work itself is not
	 * aborted; it completes and writes into the staging texture, which is then
	 * reused by the next EnqueueCopy.
	 */
	void Invalidate();

	//~ Begin FGCObject
	virtual FString GetReferencerName() const override;
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	//~ End FGCObject

protected:
	/** Source texture, held weakly so sampling no-ops once it's GC'd. */
	TWeakObjectPtr<UTexture> TextureWeak;

	/** 1x1 RTF_RGBA16f intermediate, lazily created on the first sample and reused thereafter. */
	TObjectPtr<UTextureRenderTarget2D> RenderTarget;

	/**
	 * Owns the FRHIGPUTextureReadback and the cross-thread cached sample.
	 * Registered via BeginInitResource in the constructor, released on the render
	 * thread in the destructor. Captured as a WeakPtr by sample render commands.
	 */
	TSharedPtr<FMediaViewerTextureSampleResource> SampleResource;
};

} // UE::MediaViewer::Private
