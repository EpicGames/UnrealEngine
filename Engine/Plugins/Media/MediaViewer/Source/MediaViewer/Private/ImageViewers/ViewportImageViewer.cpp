// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageViewers/ViewportImageViewer.h"

#include "Brushes/SlateImageBrush.h"
#include "Engine/TextureRenderTarget2D.h"
#include "GlobalShader.h"
#include "Math/Vector2D.h"
#include "MediaViewerUtils.h"
#include "Misc/Optional.h"
#include "Misc/TVariant.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "TextureResource.h"
#include "UnrealClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ViewportImageViewer)

namespace UE::MediaViewer::Private
{

FViewportImageViewer::FViewportImageViewer(const FMediaImageViewerInfo& InImageInfo)
	: FMediaImageViewer(InImageInfo)
	, RenderState(ERenderState::NotRendered)
	, bResizeTargetToViewport(false)
{
	DrawEffects |= ESlateDrawEffect::InvertAlpha | ESlateDrawEffect::PreMultipliedAlpha | ESlateDrawEffect::NoGamma;
}

FViewportImageViewer::~FViewportImageViewer()
{
	// Not removing bind to FCoreDelegates::OnEndFrameRT here because it can cause threading issues.
	// It will be compacted and removed automatically by the delegate.
}

bool FViewportImageViewer::IsValid() const
{
	return FMediaImageViewer::IsValid() && !!GetViewport();
}

TOptional<TVariant<FColor, FLinearColor>> FViewportImageViewer::GetPixelColor(const FIntPoint& InPixelCoords, int32 InMipLevel) const
{
	if (!SampleCache.IsValid() || !SampleCache->IsValid())
	{
		return {};
	}

	if (InPixelCoords.X < 0 || InPixelCoords.Y < 0)
	{
		SampleCache->Invalidate();
		return {};
	}

	if (InPixelCoords.X >= ImageInfo.Size.X || InPixelCoords.Y >= ImageInfo.Size.Y)
	{
		SampleCache->Invalidate();
		return {};
	}

	FIntPoint MirroredPixelCoords = InPixelCoords;

	if (EnumHasAllFlags(static_cast<EMediaImageMirrorPlane>(PaintSettings.MirrorPlanes), EMediaImageMirrorPlane::Horizontal))
	{
		MirroredPixelCoords.X = ImageInfo.Size.X - InPixelCoords.X - 1;
	}

	if (EnumHasAllFlags(static_cast<EMediaImageMirrorPlane>(PaintSettings.MirrorPlanes), EMediaImageMirrorPlane::Vertical))
	{
		MirroredPixelCoords.Y = ImageInfo.Size.Y - InPixelCoords.Y - 1;
	}

	if (const FLinearColor* PixelColor = SampleCache->GetPixelColor(MirroredPixelCoords))
	{
		TVariant<FColor, FLinearColor> PixelColorVariant;
		PixelColorVariant.Set<FLinearColor>(*PixelColor);
		PixelColorVariant.Get<FLinearColor>().A = 1.f - PixelColorVariant.Get<FLinearColor>().A;

		return PixelColorVariant;
	}

	return {};
}

TSharedPtr<FStructOnScope> FViewportImageViewer::GetCustomSettingsOnScope() const
{
	return MakeShared<FStructOnScope>(
		FViewportImageViewerSettings::StaticStruct(), 
		reinterpret_cast<uint8*>(&const_cast<FViewportImageViewerSettings&>(ViewportSettings))
	);
}

void FViewportImageViewer::PaintImage(FMediaImageSlatePaintParams& InPaintParams, const FMediaImageSlatePaintGeometry& InPaintGeometry)
{
	if (RenderState == ERenderState::NotRendered || ViewportSettings.bRealTime)
	{
		RequestRender(/* Update render target size */ true);
	}

	FMediaImageViewer::PaintImage(InPaintParams, InPaintGeometry);
}

FString FViewportImageViewer::GetReferencerName() const
{
	static const FString ReferencerName = TEXT("FViewportImageViewer");
	return ReferencerName;
}

void FViewportImageViewer::AddReferencedObjects(FReferenceCollector& InCollector)
{
	FMediaImageViewer::AddReferencedObjects(InCollector);

	InCollector.AddPropertyReferencesWithStructARO(
		FViewportImageViewerSettings::StaticStruct(),
		&ViewportSettings
	);
}

void FViewportImageViewer::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged)
{
	FMediaImageViewer::NotifyPostChange(InPropertyChangedEvent, InPropertyThatChanged);

	if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(FMediaImagePaintSettings, MirrorPlanes))
	{
		OnMirrored();
	}
}

void FViewportImageViewer::OnMirrored()
{
	if (SampleCache.IsValid())
	{
		SampleCache->Invalidate();
	}
}

void FViewportImageViewer::CreateBrush()
{
	FViewport* Viewport = GetViewport();

	if (!Viewport)
	{
		return;
	}

	ImageInfo.Size = Viewport->GetSizeXY();

	UTextureRenderTarget2D* RenderTarget = FMediaViewerUtils::CreateRenderTarget(ImageInfo.Size, /* Transparent */ false);
	ViewportSettings.RenderTarget = RenderTarget;

	Brush = MakeShared<FSlateImageBrush>(RenderTarget, FVector2D(RenderTarget->GetSurfaceWidth(), RenderTarget->GetSurfaceDepth()));

	SampleCache = MakeShared<FTextureSampleCache>(RenderTarget, RenderTarget->GetFormat());
}

void FViewportImageViewer::RenderViewport(TNotNull<FViewport*> InViewport, TNotNull<UTextureRenderTarget2D*> InRenderTarget,
	FRenderComplete InRenderComplete, bool bResizeTargetToViewport)
{
	if (bResizeTargetToViewport)
	{
		const FIntPoint ViewportSize = InViewport->GetSizeXY();

		if (ViewportSize.X != InRenderTarget->GetSurfaceWidth()
			|| ViewportSize.Y != InRenderTarget->GetSurfaceHeight())
		{
			InRenderTarget->ResizeTarget(ViewportSize.X, ViewportSize.Y);
		}
	}

	ENQUEUE_RENDER_COMMAND(CopyViewportRenderTarget)(
		[Viewport = InViewport, RenderTarget = InRenderTarget, InRenderComplete](FRHICommandListImmediate& InRHICmdList)
		{
			FTextureRenderTargetResource* RenderTargetResource = RenderTarget->GetRenderTargetResource();

			if (!RenderTargetResource)
			{
				InRenderComplete.ExecuteIfBound(false);
				return;
			}

			FRDGBuilder GraphBuilder(InRHICmdList);

			FRDGTexture* InputTexture = Viewport->GetRenderTargetTexture(GraphBuilder);

			if (!InputTexture)
			{
				InRenderComplete.ExecuteIfBound(false);
				return;
			}

			constexpr int32 MinDimension = 3;

			if (InputTexture->Desc.Extent.X < MinDimension || InputTexture->Desc.Extent.Y < MinDimension)
			{
				InRenderComplete.ExecuteIfBound(false);
				return;
			}

			FRDGTexture* OutputTexture = RenderTargetResource->GetRenderTargetTexture(GraphBuilder);

			if (!OutputTexture)
			{
				InRenderComplete.ExecuteIfBound(false);
				return;
			}

			if (OutputTexture->Desc.Extent.X < MinDimension || OutputTexture->Desc.Extent.Y < MinDimension)
			{
				InRenderComplete.ExecuteIfBound(false);
				return;
			}

			const FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

			AddDrawTexturePass(GraphBuilder, GlobalShaderMap, InputTexture, OutputTexture, FRDGDrawTextureInfo());

			GraphBuilder.Execute();

			InRenderComplete.ExecuteIfBound(true);
		}
	);
}

void FViewportImageViewer::RequestRender(bool bInResizeTargetToViewport)
{
	bInvalidatedMipTexture = true;
	RenderState = ERenderState::Invalidated;
	bResizeTargetToViewport = bResizeTargetToViewport || bInResizeTargetToViewport;

	if (!OnEndFrameRTHandle.IsValid())
	{
		OnEndFrameRTHandle = FCoreDelegates::OnEndFrameRT.AddSP(
			this,
			&FViewportImageViewer::ConditionallyRequestRender
		);
	}
}

void FViewportImageViewer::ConditionallyRequestRender()
{
	if (RenderState == ERenderState::RenderQueued)
	{
		return;
	}

	FViewport* Viewport = GetViewport();

	if (!Viewport || !::IsValid(ViewportSettings.RenderTarget))
	{
		return;
	}

	ImageInfo.Size = Viewport->GetSizeXY();

	RenderViewport(
		TNotNull<FViewport*>(Viewport), 
		TNotNull<UTextureRenderTarget2D*>(ViewportSettings.RenderTarget),
		FRenderComplete::CreateSP(this, &FViewportImageViewer::OnRenderComplete),
		bResizeTargetToViewport
	);

	RenderState = ERenderState::RenderQueued;
	bResizeTargetToViewport = false;
}

void FViewportImageViewer::OnRenderComplete(bool bInSuccess)
{
	if (bInSuccess && SampleCache.IsValid())
	{
		SampleCache->MarkDirty();
	}
}

} // UE::MediaViewer::Private
