// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImageViewer/IMediaImageViewerFactory.h"
#include "ImageViewer/MediaImageViewer.h"

#include "Delegates/IDelegateInstance.h"
#include "ImageViewers/TextureSampleCache.h"
#include "MediaViewerUtils.h"
#include "Templates/SharedPointer.h"

#include "ViewportImageViewer.generated.h"

class FTextureRenderTargetResource;
class FViewport;
class UTextureRenderTarget2D;

USTRUCT()
struct FViewportImageViewerSettings 
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Material")
	TObjectPtr<UTextureRenderTarget2D> RenderTarget = nullptr;

	/** Will render the viewport every frame. */
	UPROPERTY(EditAnywhere, Category = "Material")
	bool bRealTime = true;
};

namespace UE::MediaViewer::Private
{

class FViewportImageViewer : public FMediaImageViewer
{
public:
	FViewportImageViewer(const FMediaImageViewerInfo& InImageInfo);

	virtual ~FViewportImageViewer() override;

	//~ Begin FMediaImageViewer
	virtual bool IsValid() const override;
	virtual TOptional<TVariant<FColor, FLinearColor>> GetPixelColor(const FIntPoint& InPixelCoords, int32 InMipLevel) const override;
	virtual TSharedPtr<FStructOnScope> GetCustomSettingsOnScope() const override;
	virtual void OnMirrored() override;
	//~ End FMediaImageViewer

	//~ Begin FGCObject
	virtual FString GetReferencerName() const override;
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	//~ End FGCObject

	//~ Begin FNotifyHook
	virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged) override;
	//~ End FNotifyHook

protected:
	enum class ERenderState : uint8
	{
		NotRendered,
		RenderQueued,
		Invalidated
	};

	static void RenderViewport(TNotNull<FViewport*> InViewport, TNotNull<UTextureRenderTarget2D*> InRenderTarget, 
		FRenderComplete InRenderComplete, bool bResizeTargetToViewport = true);

	FViewportImageViewerSettings ViewportSettings;
	TSharedPtr<FTextureSampleCache> SampleCache;
	FDelegateHandle OnEndFrameRTHandle;
	ERenderState RenderState;
	bool bResizeTargetToViewport;

	virtual FViewport* GetViewport() const = 0;

	void CreateBrush();

	void RequestRender(bool bInResizeTargetToViewport);
	
	void ConditionallyRequestRender();

	void OnRenderComplete(bool bInSuccess);

	//~ Begin FMediaImageViewer
	virtual void PaintImage(FMediaImageSlatePaintParams& InPaintParams, const FMediaImageSlatePaintGeometry& InPaintGeometry) override;
	//~ End FMediaImageViewer
};

} // UE::MediaViewer::Private
