// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/LevelEditorViewportGroup.h"

#include "ImageViewers/LevelEditorViewportImageViewer.h"
#include "LevelEditor.h"
#include "Library/MediaViewerLibraryItem.h"
#include "Modules/ModuleManager.h"
#include "Slate/SceneViewport.h"
#include "SLevelViewport.h"

#define LOCTEXT_NAMESPACE "LevelEditorViewportGroup"

namespace UE::MediaViewer::Private
{

FLevelEditorViewportGroup::FLevelEditorViewportGroup(const TSharedRef<IMediaViewerLibrary>& InLibrary)
	: FLevelEditorViewportGroup(InLibrary, FGuid::NewGuid())
{
}

FLevelEditorViewportGroup::FLevelEditorViewportGroup(const TSharedRef<IMediaViewerLibrary>& InLibrary,
	const FGuid& InGuid)
	: FMediaViewerLibraryDynamicGroup(
		InLibrary,
		InGuid,
		LOCTEXT("LevelEditorViewports", "Editor Viewports"),
		LOCTEXT("LevelEditorViewportsTooltip", "The viewports available in the Level Editor."),
		FGenerateItems::CreateStatic(&FLevelEditorViewportGroup::GetLevelEditorViewportItems)
	)
{
}

TArray<TSharedRef<FMediaViewerLibraryItem>> FLevelEditorViewportGroup::GetLevelEditorViewportItems()
{
	TArray<TSharedRef<FMediaViewerLibraryItem>> LevelEditorViewportItems;

	FLevelEditorModule& LevelEditorModule = FModuleManager::Get().GetModuleChecked<FLevelEditorModule>("LevelEditor");

	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();

	if (!LevelEditor.IsValid())
	{
		return LevelEditorViewportItems;
	}

	TMap<int32, TArray<TSharedPtr<SLevelViewport>>> Viewports = LevelEditor->GetViewportsByTab();
	
	TSet<FGuid> AddedViewportIds;
	AddedViewportIds.Reserve(Viewports.Num());

	for (const TPair<int32, TArray<TSharedPtr<SLevelViewport>>>& ViewportTabPair : Viewports)
	{
		for (int32 Index = 0; Index < ViewportTabPair.Value.Num(); ++Index)
		{
			TSharedPtr<FSceneViewport> ActiveViewport = ViewportTabPair.Value[Index]->GetSharedActiveViewport();

			if (!ActiveViewport.IsValid())
			{
				continue;
			}

			const FIntPoint Size = ActiveViewport->GetSize();

			if (Size.X < 2 || Size.Y < 2)
			{
				continue;
			}

			const FString ConfigKey = FString::FromInt(ViewportTabPair.Key) + TEXT("_") + ViewportTabPair.Value[Index]->GetConfigKey().ToString();

			const FGuid ViewportId = FLevelEditorViewportImageViewer::FItem::GetIdForViewport(
				ConfigKey,
				/* Create id if invalid */ false
			);

			bool bAlreadyInSet = false;
			AddedViewportIds.Add(ViewportId, &bAlreadyInSet);

			if (bAlreadyInSet)
			{
				continue;
			}

			if (ViewportId.IsValid())
			{
				LevelEditorViewportItems.Add(MakeShared<FLevelEditorViewportImageViewer::FItem>(ViewportId, ConfigKey));
			}
		}
	}

	return LevelEditorViewportItems;
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
