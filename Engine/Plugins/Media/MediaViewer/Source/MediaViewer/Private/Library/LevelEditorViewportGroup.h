// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Library/MediaViewerLibraryDynamicGroup.h"

namespace UE::MediaViewer::Private
{

/**
 * A group that generates entries based on available level editor viewports.
 */
struct FLevelEditorViewportGroup : public FMediaViewerLibraryDynamicGroup
{
	FLevelEditorViewportGroup(const TSharedRef<IMediaViewerLibrary>& InLibrary);
	FLevelEditorViewportGroup(const TSharedRef<IMediaViewerLibrary>& InLibrary, const FGuid& InId);

	virtual ~FLevelEditorViewportGroup() override = default;

protected:
	static TArray<TSharedRef<FMediaViewerLibraryItem>> GetLevelEditorViewportItems();
};

} // UE::MediaViewer::Private
