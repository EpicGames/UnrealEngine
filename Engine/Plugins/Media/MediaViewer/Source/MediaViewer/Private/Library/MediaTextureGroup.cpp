// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/MediaTextureGroup.h"

#include "ImageViewers/MediaTextureImageViewer.h"
#include "Library/MediaViewerLibraryItem.h"
#include "MediaTexture.h"
#include "MediaViewerUtils.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "MediaTextureGroup"

namespace UE::MediaViewer::Private
{

FMediaTextureGroup::FMediaTextureGroup(const TSharedRef<IMediaViewerLibrary>& InLibrary)
	: FMediaTextureGroup(InLibrary, FGuid::NewGuid())
{
}

FMediaTextureGroup::FMediaTextureGroup(const TSharedRef<IMediaViewerLibrary>& InLibrary,
	const FGuid& InGuid)
	: FMediaViewerLibraryDynamicGroup(
		InLibrary,
		InGuid,
		LOCTEXT("MediaTextures", "Media Textures"),
		LOCTEXT("MediaTexturesTooltip", "The available media textures."),
		FGenerateItems::CreateStatic(&FMediaTextureGroup::GetMediaTextureItems)
	)
{
}

TArray<TSharedRef<FMediaViewerLibraryItem>> FMediaTextureGroup::GetMediaTextureItems()
{
	TArray<TSharedRef<FMediaViewerLibraryItem>> MediaTextureItems;

	TSharedRef<FMediaTextureImageViewer::FFactory> Factory = MakeShared<FMediaTextureImageViewer::FFactory>();

	for (UMediaTexture* MediaTexture : TObjectRange<UMediaTexture>())
	{
		if (MediaTexture->IsTemplate())
		{
			continue;
		}

		if (MediaTexture->HasAssetUserDataOfClass(UMediaViewerUserData::StaticClass()))
		{
			continue;
		}

		constexpr int32 MinimumSize = 3;

		if (MediaTexture->GetSurfaceWidth() < MinimumSize || MediaTexture->GetSurfaceHeight() < MinimumSize)
		{
			continue;
		}

		if (TSharedPtr<FMediaViewerLibraryItem> NewItem = Factory->CreateLibraryItem(TNotNull<UMediaTexture*>(MediaTexture)))
		{
			MediaTextureItems.Add(NewItem.ToSharedRef());
		}
	}

	return MediaTextureItems;
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
