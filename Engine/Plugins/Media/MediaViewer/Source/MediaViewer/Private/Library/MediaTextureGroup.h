// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Library/MediaViewerLibraryDynamicGroup.h"

namespace UE::MediaViewer::Private
{

/**
 * A group that generates entries based on available UMediaTextures.
*/
struct FMediaTextureGroup : public FMediaViewerLibraryDynamicGroup
{
	FMediaTextureGroup(const TSharedRef<IMediaViewerLibrary>& InLibrary);
	FMediaTextureGroup(const TSharedRef<IMediaViewerLibrary>& InLibrary, const FGuid& InId);

	virtual ~FMediaTextureGroup() override = default;

protected:
	static TArray<TSharedRef<FMediaViewerLibraryItem>> GetMediaTextureItems();
};

} // UE::MediaViewer::Private
