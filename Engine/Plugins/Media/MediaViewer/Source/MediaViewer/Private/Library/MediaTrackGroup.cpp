// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/MediaTrackGroup.h"

#include "ImageViewers/MediaTextureImageViewer.h"
#include "Library/MediaViewerLibraryItem.h"
#include "MediaPlayer.h"
#include "MediaTexture.h"
#include "MediaViewerUtils.h"
#include "MovieSceneMediaSection.h"
#include "MovieSceneMediaTrack.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "MediaTrackGroup"

namespace UE::MediaViewer::Private
{

	FMediaTrackGroup::FMediaTrackGroup(const TSharedRef<IMediaViewerLibrary>& InLibrary)
		: FMediaTrackGroup(InLibrary, FGuid::NewGuid())
	{
	}

	FMediaTrackGroup::FMediaTrackGroup(const TSharedRef<IMediaViewerLibrary>& InLibrary,
		const FGuid& InGuid)
		: FMediaViewerLibraryDynamicGroup(
			InLibrary,
			InGuid,
			LOCTEXT("MediaTracks", "Media Tracks"),
			LOCTEXT("MediaTracksTooltip", "The media textures from media tracks."),
			FGenerateItems::CreateStatic(&FMediaTrackGroup::GetMediaTrackItems)
		)
	{
	}

	TArray<TSharedRef<FMediaViewerLibraryItem>> FMediaTrackGroup::GetMediaTrackItems()
	{
		TArray<TSharedRef<FMediaViewerLibraryItem>> MediaTrackItems;

		TSharedRef<FMediaTextureImageViewer::FFactory> Factory = MakeShared<FMediaTextureImageViewer::FFactory>();

		for (UMovieSceneMediaSection* MediaSection : TObjectRange<UMovieSceneMediaSection>())
		{
			if (MediaSection->IsTemplate() || !MediaSection->MediaTexture)
			{
				continue;
			}

			if (TSharedPtr<FMediaViewerLibraryItem> NewItem = Factory->CreateLibraryItem(TNotNull<UMediaTexture*>(MediaSection->MediaTexture)))
			{
				if (UMovieSceneMediaTrack* MediaTrack = Cast<UMovieSceneMediaTrack>(MediaSection->GetOuter()))
				{
					const FText DisplayName = MediaTrack->GetDisplayName();

					if (!DisplayName.IsEmpty())
					{
						NewItem->Name = DisplayName;
					}
				}

				MediaTrackItems.Add(NewItem.ToSharedRef());
			}
		}

		return MediaTrackItems;
	}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
