// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Library/MediaViewerLibraryDynamicGroup.h"

namespace UE::MediaViewer::Private
{

	/**
	 * A group that generates entries based on available UMediaTracks.
	*/
	struct FMediaTrackGroup : public FMediaViewerLibraryDynamicGroup
	{
		FMediaTrackGroup(const TSharedRef<IMediaViewerLibrary>& InLibrary);
		FMediaTrackGroup(const TSharedRef<IMediaViewerLibrary>& InLibrary, const FGuid& InId);

		virtual ~FMediaTrackGroup() override = default;

	protected:
		static TArray<TSharedRef<FMediaViewerLibraryItem>> GetMediaTrackItems();
	};

} // UE::MediaViewer::Private
