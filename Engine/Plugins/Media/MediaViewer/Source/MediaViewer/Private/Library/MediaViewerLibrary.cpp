// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/MediaViewerLibrary.h"

#include "Containers/UnrealString.h"
#include "Library/MediaViewerLibraryDynamicGroup.h"
#include "Library/MediaViewerLibraryGroup.h"
#include "Library/MediaViewerLibraryItem.h"
#include "Misc/Guid.h"

#define LOCTEXT_NAMESPACE "MediaViewerLibrary"

namespace UE::MediaViewer::Private
{
	constexpr FGuid DefaultGroupId = FGuid(0xD42C17B4, 0x47939576, 0x3BDB9594, 0x5101041A);
	constexpr FGuid HistoryGroupId = FGuid(0x35C7E8CA, 0x2B6D1A76, 0xA9EAFE80, 0xA2B12B41);
	constexpr FGuid SnapshotGroupId = FGuid(0X11F83073, 0X7FF4BC41, 0XA4FFE792, 0XD073F41F);
}

namespace UE::MediaViewer::Private
{

const FGuid& FMediaViewerLibrary::GetDefaultGroupId() const
{
	return DefaultGroupId;
}

const FGuid& FMediaViewerLibrary::GetHistoryGroupId() const
{
	return HistoryGroupId;
}

const FGuid& FMediaViewerLibrary::GetSnapshotsGroupId() const
{
	return SnapshotGroupId;
}

FMediaViewerLibrary::FMediaViewerLibrary()
{
	Groups.Add(MakeShared<FMediaViewerLibraryGroup>(
		GetDefaultGroupId(),
		LOCTEXT("DefaultGroup", "Pinned"),
		LOCTEXT("DefaultGroupTooltip", "Pinned items."),
		/* Dynamic */ false
	));

	Groups.Add(MakeShared<FMediaViewerLibraryGroup>(
		GetSnapshotsGroupId(),
		LOCTEXT("Snapshots", "Snapshots"),
		LOCTEXT("SnapshotsTooltip", "Snapshots taken with the Media Viewer tool."),
		/* Dynamic */ false
	));

	Groups.Add(MakeShared<FMediaViewerLibraryGroup>(
		GetHistoryGroupId(),
		LOCTEXT("History", "History"),
		FText::Format(
			LOCTEXT("HistoryTooltip", "Up to the last {0} viewed items."),
			FText::AsNumber(UE::MediaViewer::Private::MaxHistoryEntries)
		),
		/* Dynamic */ false
	));
}

bool FMediaViewerLibrary::CanDragDropGroup(const FGuid& InGroupId) const
{
	TSharedPtr<FMediaViewerLibraryGroup> Group = GetGroup(InGroupId);

	return Group.IsValid() && !Group->IsDynamic();
}

bool FMediaViewerLibrary::CanDragDropItem(const FGroupItem& InItem) const
{
	TSharedPtr<FMediaViewerLibraryGroup> Group = GetGroup(InItem.GroupId);

	return Group.IsValid() && !Group->IsDynamic();
}

const TArray<TSharedRef<const FMediaViewerLibraryGroup>>& FMediaViewerLibrary::GetGroups() const
{
	return reinterpret_cast<const TArray<TSharedRef<const FMediaViewerLibraryGroup>>&>(Groups);
}

bool FMediaViewerLibrary::AddGroup(const TSharedRef<FMediaViewerLibraryGroup>& InNewGroup)
{
	if (GetGroup(InNewGroup->GetId()).IsValid())
	{
		return false;
	}

	Groups.Add(InNewGroup);

	if (InNewGroup->IsDynamic())
	{
		StaticCastSharedRef<FMediaViewerLibraryDynamicGroup>(InNewGroup)->GetOnItemsUpdated().AddSP(
			this,
			&FMediaViewerLibrary::OnChanged,
			EChangeType::GroupItemsChanged
		);
	}

	OnChanged(FMediaViewerLibrary::EChangeType::GroupAdded);

	return true;
}

TSharedPtr<FMediaViewerLibraryGroup> FMediaViewerLibrary::GetGroup(const FGuid& InGroupId) const
{
	for (const TSharedRef<FMediaViewerLibraryGroup>& Group : Groups)
	{
		if (Group->GetId() == InGroupId)
		{
			return Group;
		}
	}

	return nullptr;
}

int32 FMediaViewerLibrary::EmptyGroup(const FGuid& InGroupId)
{
	TSharedPtr<FMediaViewerLibraryGroup> Group = GetGroup(InGroupId);

	if (!Group.IsValid())
	{
		return 0;
	}

	const int32 Count = Group->Empty();	

	OnChanged(FMediaViewerLibrary::EChangeType::ItemRemoved);

	return Count;
}

bool FMediaViewerLibrary::CanRemoveGroup(const FGuid& InGroupIdToRemove) const
{
	if (InGroupIdToRemove == GetDefaultGroupId() || InGroupIdToRemove == GetHistoryGroupId() || InGroupIdToRemove == GetSnapshotsGroupId())
	{
		return false;
	}

	TSharedPtr<FMediaViewerLibraryGroup> Group = GetGroup(InGroupIdToRemove);

	return Group.IsValid() && !Group->IsDynamic();
}

TSharedPtr<FMediaViewerLibraryGroup> FMediaViewerLibrary::RemoveGroup(const FGuid& InGroupIdToRemove)
{
	if (!CanRemoveGroup(InGroupIdToRemove))
	{
		return nullptr;
	}

	int32 GroupToRemoveIndex = INDEX_NONE;

	for (int32 Index = 0; Index < Groups.Num(); ++Index)
	{
		if (Groups[Index]->GetId() == InGroupIdToRemove)
		{
			GroupToRemoveIndex = Index;
			break;
		}
	}

	if (GroupToRemoveIndex == INDEX_NONE)
	{
		return nullptr;
	}

	TSharedPtr<FMediaViewerLibraryGroup> GroupToRemove = Groups[GroupToRemoveIndex];

	if (GroupToRemove->IsDynamic())
	{
		StaticCastSharedPtr<FMediaViewerLibraryDynamicGroup>(GroupToRemove)->GetOnItemsUpdated().RemoveAll(this);
	}

	Groups.RemoveAt(GroupToRemoveIndex);

	OnChanged(FMediaViewerLibrary::EChangeType::GroupRemoved);

	return GroupToRemove;
}

TSharedPtr<FMediaViewerLibraryItem> FMediaViewerLibrary::FindItemByValue(FName InItemType, const FString& InStringValue) const
{
	for (const TPair<FGuid, TSharedRef<FMediaViewerLibraryItem>>& ItemPair : Items)
	{
		const TSharedRef<FMediaViewerLibraryItem>& Item = ItemPair.Value;

		if (Item->GetItemType() == InItemType && Item->GetStringValue() == InStringValue)
		{
			return Item;
		}
	}

	return nullptr;
}

TSharedPtr<FMediaViewerLibraryItem> FMediaViewerLibrary::GetItem(const FGuid& InItemId) const
{
	if (const TSharedRef<FMediaViewerLibraryItem>* ItemPtr = Items.Find(InItemId))
	{
		return *ItemPtr;
	}

	return nullptr;
}

TSharedPtr<FMediaViewerLibraryGroup> FMediaViewerLibrary::GetItemGroup(const FGuid& InItemId) const
{
	for (const TSharedRef<FMediaViewerLibraryGroup>& Group : Groups)
	{
		if (Group->GetId() == GetHistoryGroupId() || Group->GetId() == GetSnapshotsGroupId())
		{
			continue;
		}

		if (Group->ContainsItem(InItemId))
		{
			return Group;
		}
	}

	return nullptr;
}

bool FMediaViewerLibrary::AddItem(const TSharedRef<FMediaViewerLibraryItem>& InNewItem)
{
	const FGuid& ItemId = InNewItem->GetId();

	if (!Items.Contains(ItemId))
	{
		Items.Add(ItemId, InNewItem);
		return true;
	}

	return false;
}

bool FMediaViewerLibrary::AddItemToGroup(const TSharedRef<FMediaViewerLibraryItem>& InNewItem, TOptional<FGuid> InTargetGroupId,
	int32 InIndex)
{
	const FGuid& ItemId = InNewItem->GetId();

	if (InTargetGroupId != HistoryGroupId && GetItemGroup(InNewItem->GetId()).IsValid())
	{
		return false;
	}

	TSharedPtr<FMediaViewerLibraryGroup> TargetGroup = GetGroup(InTargetGroupId.Get(GetDefaultGroupId()));

	if (!TargetGroup.IsValid() || TargetGroup->IsDynamic())
	{
		return false;
	}

	if (TargetGroup->ContainsItem(ItemId))
	{
		return false;
	}

	Items.Add(ItemId, InNewItem);

	TargetGroup->AddItem(ItemId, InIndex);

	OnChanged(FMediaViewerLibrary::EChangeType::ItemGroupChanged);

	return true;
}

bool FMediaViewerLibrary::AddItemBelowItem(const TSharedRef<FMediaViewerLibraryItem>& InNewItem, const FGroupItem& InTargetItem)
{
	const FGuid& ItemId = InNewItem->GetId();

	if (ItemId == InTargetItem.ItemId)
	{
		return false;
	}

	if (GetItemGroup(InNewItem->GetId()).IsValid())
	{
		return false;
	}

	TSharedPtr<FMediaViewerLibraryGroup> TargetGroup = GetGroup(InTargetItem.GroupId);

	if (!TargetGroup.IsValid() || TargetGroup->IsDynamic())
	{
		return false;
	}

	const int32 TargetIndex = TargetGroup->FindItemIndex(InTargetItem.ItemId);

	if (TargetIndex == INDEX_NONE)
	{
		return false;
	}

	Items.Add(ItemId, InNewItem);

	TargetGroup->AddItem(ItemId, TargetIndex + 1);

	OnChanged(FMediaViewerLibrary::EChangeType::ItemGroupChanged);

	return true;
}

bool FMediaViewerLibrary::MoveItemToGroup(const FGroupItem& InItemToMove, const FGuid& InTargetGroupId, int32 InIndex)
{
	if (InItemToMove.GroupId == InTargetGroupId)
	{
		return false;
	}
	
	TSharedPtr<FMediaViewerLibraryGroup> TargetGroup = GetGroup(InTargetGroupId);

	if (!TargetGroup.IsValid() || TargetGroup->IsDynamic())
	{
		return false;
	}

	if (TargetGroup->ContainsItem(InItemToMove.ItemId))
	{
		return false;
	}

	if (TSharedPtr<FMediaViewerLibraryGroup> CurrentGroup = GetGroup(InItemToMove.GroupId))
	{
		CurrentGroup->RemoveItem(InItemToMove.ItemId);
	}

	TargetGroup->AddItem(InItemToMove.ItemId, InIndex);

	OnChanged(FMediaViewerLibrary::EChangeType::ItemGroupChanged);

	return true;
}

bool FMediaViewerLibrary::MoveItemWithinGroup(const FGroupItem& InItemToMove, int32 InIndex)
{
	TSharedPtr<FMediaViewerLibraryGroup> Group = GetGroup(InItemToMove.GroupId);

	if (!Group.IsValid() || Group->IsDynamic())
	{
		return false;
	}

	if (InIndex < 0 || InIndex > Group->GetItems().Num())
	{
		return false;
	}

	const int32 ItemIndex = Group->FindItemIndex(InItemToMove.ItemId);

	if (ItemIndex == INDEX_NONE || ItemIndex == InIndex)
	{
		return false;
	}

	if (ItemIndex < InIndex)
	{
		// Removing from the group will push the target down one.
		--InIndex;
	}

	Group->RemoveItem(InItemToMove.ItemId);
	Group->AddItem(InItemToMove.ItemId, InIndex);

	OnChanged(FMediaViewerLibrary::EChangeType::ItemGroupChanged);

	return true;
}

bool FMediaViewerLibrary::MoveItemBelowItem(const FGroupItem& InItemToMove, const FGroupItem& InTargetItem)
{
	if (InItemToMove.ItemId == InTargetItem.ItemId)
	{
		return false;
	}

	TSharedPtr<FMediaViewerLibraryGroup> SourceGroup = GetGroup(InItemToMove.GroupId);

	if (!SourceGroup.IsValid() || SourceGroup->IsDynamic())
	{
		return false;
	}

	TSharedPtr<FMediaViewerLibraryGroup> TargetGroup = GetGroup(InTargetItem.GroupId);

	if (!TargetGroup.IsValid() || TargetGroup->IsDynamic())
	{
		return false;
	}

	const int32 TargetIndex = TargetGroup->FindItemIndex(InTargetItem.ItemId);

	if (TargetIndex == INDEX_NONE)
	{
		return false;
	}

	if (InItemToMove.GroupId == InTargetItem.GroupId)
	{
		return MoveItemWithinGroup(InItemToMove, TargetIndex + 1);
	}

	SourceGroup->RemoveItem(InItemToMove.ItemId);

	TargetGroup->AddItem(InItemToMove.ItemId, TargetIndex + 1);

	OnChanged(FMediaViewerLibrary::EChangeType::ItemGroupChanged);

	return true;
}

bool FMediaViewerLibrary::CanRemoveItemFromGroup(const FGroupItem& InItemToRemove) const
{
	TSharedPtr<FMediaViewerLibraryGroup> Group = GetGroup(InItemToRemove.GroupId);

	return Group.IsValid() && !Group->IsDynamic() && Group->ContainsItem(InItemToRemove.ItemId);
}

bool FMediaViewerLibrary::RemoveItemFromGroup(const FGroupItem& InItemToRemove)
{
	TSharedPtr<FMediaViewerLibraryGroup> Group = GetGroup(InItemToRemove.GroupId);

	if (!Group.IsValid() || Group->IsDynamic())
	{
		return false;
	}

	if (!Group->RemoveItem(InItemToRemove.ItemId))
	{
		return false;
	}

	OnChanged(FMediaViewerLibrary::EChangeType::ItemGroupChanged);
	return true;
}

bool FMediaViewerLibrary::CanRemoveItem(const FGuid& InItemIdToRemove) const
{
	const TSharedRef<FMediaViewerLibraryItem>* ItemPtr = Items.Find(InItemIdToRemove);

	if (!ItemPtr)
	{
		return false;
	}

	for (const TSharedRef<FMediaViewerLibraryGroup>& Group : Groups)
	{
		if (!Group->IsDynamic())
		{
			continue;
		}

		if (Group->ContainsItem(InItemIdToRemove))
		{
			return false;
		}
	}

	return true;
}

TSharedPtr<FMediaViewerLibraryItem> FMediaViewerLibrary::RemoveItem(const FGuid& InItemIdToRemove)
{
	if (!CanRemoveItem(InItemIdToRemove))
	{
		return nullptr;
	}

	TSharedRef<FMediaViewerLibraryItem> ItemToRemove = Items[InItemIdToRemove];

	for (const TSharedRef<FMediaViewerLibraryGroup>& Group : Groups)
	{
		if (Group->IsDynamic())
		{
			continue;
		}

		Group->RemoveItem(InItemIdToRemove);
	}

	Items.Remove(InItemIdToRemove);

	OnChanged(FMediaViewerLibrary::EChangeType::ItemRemoved);

	return ItemToRemove;
}

IMediaViewerLibrary::FOnChanged::RegistrationType& FMediaViewerLibrary::GetOnChanged()
{
	return OnChangedDelegate;
}

void FMediaViewerLibrary::RemoveInvalidGroupItems(const FGuid& InGroup)
{
	TSharedPtr<FMediaViewerLibraryGroup> Group = GetGroup(InGroup);

	if (!Group.IsValid())
	{
		return;
	}

	const TArray<FGuid>& GroupItems = Group->GetItems();

	for (int32 Index = GroupItems.Num() - 1; Index >= 0; --Index)
	{
		TSharedPtr<FMediaViewerLibraryItem> Item = GetItem(GroupItems[Index]);

		if (!Item.IsValid())
		{
			Group->RemoveItemAt(Index);
		}
	}
}

void FMediaViewerLibrary::OnChanged(FMediaViewerLibrary::EChangeType InChangeType)
{
	OnChangedDelegate.Broadcast(SharedThis(this), InChangeType);
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
