// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Library/IMediaViewerLibrary.h"
#include "Templates/SharedPointer.h"

#include "Containers/Array.h"
#include "Containers/Map.h"

class UMediaViewerLibraryIni;
struct FMediaViewerLibraryGroup;
struct FMediaViewerLibraryItem;

namespace UE::MediaViewer::Private
{

constexpr int32 MaxHistoryEntries = 20;

/**
 * Implementation of @see IMediaViewerLibrary.
 */
class FMediaViewerLibrary : public IMediaViewerLibrary, public TSharedFromThis<FMediaViewerLibrary>
{
	friend class ::UMediaViewerLibraryIni;

public:
	FMediaViewerLibrary();

	virtual ~FMediaViewerLibrary() = default;

	bool CanDragDropGroup(const FGuid& InGroupId) const;

	bool CanDragDropItem(const FGroupItem& InItem) const;

	//~ Begin FMediaViewerLibrary
	virtual const FGuid& GetDefaultGroupId() const override;
	virtual const FGuid& GetHistoryGroupId() const override;
	virtual const FGuid& GetSnapshotsGroupId() const override;
	virtual const TArray<TSharedRef<const FMediaViewerLibraryGroup>>& GetGroups() const override;
	virtual bool AddGroup(const TSharedRef<FMediaViewerLibraryGroup>& InNewGroup) override;
	virtual TSharedPtr<FMediaViewerLibraryGroup> GetGroup(const FGuid& InGroupIdToRemove) const override;
	virtual int32 EmptyGroup(const FGuid& InGroupId) override;
	virtual void RemoveInvalidGroupItems(const FGuid& InGroup) override;
	virtual bool CanRemoveGroup(const FGuid& InGroupIdToRemove) const override;
	virtual TSharedPtr<FMediaViewerLibraryGroup> RemoveGroup(const FGuid& InGroupIdToRemove) override;
	virtual TSharedPtr<FMediaViewerLibraryItem> FindItemByValue(FName InItemType, const FString& InStringValue) const override;
	virtual TSharedPtr<FMediaViewerLibraryItem> GetItem(const FGuid& InItemId) const override;
	virtual TSharedPtr<FMediaViewerLibraryGroup> GetItemGroup(const FGuid& InItemId) const override;
	virtual bool AddItem(const TSharedRef<FMediaViewerLibraryItem>& InNewItem) override;
	virtual bool AddItemToGroup(const TSharedRef<FMediaViewerLibraryItem>& InNewItem, TOptional<FGuid> InTargetGroupId = {},
		int32 InIndex = INDEX_NONE) override;
	virtual bool AddItemBelowItem(const TSharedRef<FMediaViewerLibraryItem>& InNewItem, const FGroupItem& InTargetItem) override;
	virtual bool MoveItemToGroup(const FGroupItem& InItemToMove, const FGuid& InTargetGroupId, int32 InIndex = INDEX_NONE) override;
	virtual bool MoveItemWithinGroup(const FGroupItem& InItemToMove, int32 InIndex) override;
	virtual bool MoveItemBelowItem(const FGroupItem& InItemToMove, const FGroupItem& InTargetItem) override;
	virtual bool CanRemoveItemFromGroup(const FGroupItem& InItemToRemove) const override;
	virtual bool RemoveItemFromGroup(const FGroupItem& InItemToRemove) override;
	virtual bool CanRemoveItem(const FGuid& InItemIdToRemove) const override;
	virtual TSharedPtr<FMediaViewerLibraryItem> RemoveItem(const FGuid& InItemIdToRemove) override;
	virtual FOnChanged::RegistrationType& GetOnChanged() override;
	//~ End FMediaViewerLibrary

protected:
	TArray<TSharedRef<FMediaViewerLibraryGroup>> Groups;
	TMap<FGuid, TSharedRef<FMediaViewerLibraryItem>> Items;
	FOnChanged OnChangedDelegate;

	void OnChanged(FMediaViewerLibrary::EChangeType InChangeType);
};

} // UE::MediaViewer::Private