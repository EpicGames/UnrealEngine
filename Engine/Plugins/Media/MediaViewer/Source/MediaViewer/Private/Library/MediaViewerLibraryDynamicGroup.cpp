// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/MediaViewerLibraryDynamicGroup.h"

#include "Library/MediaViewerLibrary.h"
#include "Library/MediaViewerLibraryItem.h"

#define LOCTEXT_NAMESPACE "MediaViewerLibraryDynamicGroup"

namespace UE::MediaViewer
{

FMediaViewerLibraryDynamicGroup::FMediaViewerLibraryDynamicGroup(const TSharedRef<IMediaViewerLibrary>& InLibrary, const FText& InName,
	const FText& InToolTip, FGenerateItems InItemGenerator)
	: FMediaViewerLibraryDynamicGroup(InLibrary, FGuid::NewGuid(), InName, InToolTip, InItemGenerator)
{
}

FMediaViewerLibraryDynamicGroup::FMediaViewerLibraryDynamicGroup(const TSharedRef<IMediaViewerLibrary>& InLibrary, const FGuid& InId,
	const FText& InName, const FText& InToolTip, FGenerateItems InItemGenerator)
	: FMediaViewerLibraryGroup(InId, InName, InToolTip, /* Dynamic */ true)
	, LibraryWeak(InLibrary)
	, GenerateItemsDelegate(InItemGenerator)
{
	check(GenerateItemsDelegate.IsBound());
}

void FMediaViewerLibraryDynamicGroup::UpdateItems()
{
	TOptional<TArray<FGuid>> NewItems = GetUpdatedIs(Items);

	if (!NewItems.IsSet())
	{
		return;
	}

	Items = NewItems.GetValue();

	OnItemsUpdated.Broadcast();
}

FSimpleMulticastDelegate::RegistrationType& FMediaViewerLibraryDynamicGroup::GetOnItemsUpdated()
{
	return OnItemsUpdated;
}

TOptional<TArray<FGuid>> FMediaViewerLibraryDynamicGroup::GetUpdatedIs(const TArray<FGuid>& InCurrentIds)
{
	// Check if we still have a valid Library
	TSharedPtr<IMediaViewerLibrary> Library = LibraryWeak.Pin();

	if (!Library.IsValid())
	{
		return {};
	}

	// Populate map of current items
	TMap<FString, FGuid> ExistingItems;

	for (const FGuid& CurrentId : InCurrentIds)
	{
		if (TSharedPtr<FMediaViewerLibraryItem> CurrentItem = Library->GetItem(CurrentId))
		{
			ExistingItems.Add(CurrentItem->GetStringValue(), CurrentItem->GetId());
		}
	}

	// Generate new id list
	const TArray<TSharedRef<FMediaViewerLibraryItem>> UpdatedItems = GenerateItemsDelegate.Execute();

	TSet<FGuid> UpdatedIds;
	UpdatedIds.Reserve(UpdatedItems.Num());

	TSet<FString> StringValues;
	StringValues.Reserve(UpdatedItems.Num());

	bool bHasNewItem = false;

	for (const TSharedRef<FMediaViewerLibraryItem>& UpdatedItem : UpdatedItems)
	{
		const FString StringValue = UpdatedItem->GetStringValue();

		// Ensure no duplicates
		if (StringValues.Contains(StringValue))
		{
			continue;
		}

		StringValues.Add(StringValue);

		if (const FGuid* ExistingId = ExistingItems.Find(StringValue))
		{
			// Ensure no duplicates
			if (UpdatedIds.Contains(*ExistingId))
			{
				continue;
			}

			ExistingItems.Remove(StringValue);
			UpdatedIds.Add(*ExistingId);
		}
		else
		{
			Library->AddItem(UpdatedItem);
			UpdatedIds.Add(UpdatedItem->GetId());
			bHasNewItem = true;
		}
	}

	// No new items and no deleted items
	if (!bHasNewItem && ExistingItems.IsEmpty())
	{
		return {};
	}

	// Remove invalidated items
	for (const TPair<FString, FGuid>& ItemPair : ExistingItems)
	{
		Library->RemoveItem(ItemPair.Value);
	}

	return UpdatedIds.Array();
}

} // UE::MediaViewer

#undef LOCTEXT_NAMESPACE
