// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/MediaViewerLibraryEntry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaViewerLibraryEntry)

#define LOCTEXT_NAMESPACE "MediaViewerLibraryEntry"

FMediaViewerLibraryEntry::FMediaViewerLibraryEntry()
	: FMediaViewerLibraryEntry(FText::GetEmpty(), FText::GetEmpty())
{
}

FMediaViewerLibraryEntry::FMediaViewerLibraryEntry(const FText& InName, const FText& InToolTip)
	: FMediaViewerLibraryEntry(FGuid::NewGuid(), InName, InToolTip)
{
}

FMediaViewerLibraryEntry::FMediaViewerLibraryEntry(const FGuid& InId, const FText& InName, const FText& InToolTip)
	: Name(InName)
	, ToolTip(InToolTip)
	, Id(InId)
{
}

const FGuid& FMediaViewerLibraryEntry::GetId() const
{
	return Id;
}

void FMediaViewerLibraryEntry::InvalidateId()
{
	Id.Invalidate();
}

#undef LOCTEXT_NAMESPACE
