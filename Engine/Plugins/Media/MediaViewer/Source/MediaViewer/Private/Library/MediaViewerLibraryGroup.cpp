// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/MediaViewerLibraryGroup.h"

#include "Containers/Array.h"
#include "Library/MediaViewerLibraryItem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaViewerLibraryGroup)

#define LOCTEXT_NAMESPACE "MediaViewerLibraryGroup"

FMediaViewerLibraryGroup::FMediaViewerLibraryGroup()
	: FMediaViewerLibraryGroup(FText::GetEmpty(), FText::GetEmpty(), /* Dynamic */ true)
{
}

FMediaViewerLibraryGroup::FMediaViewerLibraryGroup(const FText& InName, const FText& InToolTip, bool bDynamic)
	: FMediaViewerLibraryGroup(FGuid::NewGuid(), InName, InToolTip, bDynamic)
{
}

FMediaViewerLibraryGroup::FMediaViewerLibraryGroup(const FGuid& InId, const FText& InName, const FText& InToolTip, bool bDynamic)
	: FMediaViewerLibraryEntry(InId, InName, InToolTip)
	, bDynamic(bDynamic)
{
}

FMediaViewerLibraryGroup::FMediaViewerLibraryGroup(FPrivateToken&& InPrivateToken, const FMediaViewerLibraryGroup& InSavedGroup)
	: FMediaViewerLibraryGroup(InSavedGroup.GetId(), InSavedGroup.Name, InSavedGroup.ToolTip, /* Transient */ false)
{
}

const TArray<FGuid>& FMediaViewerLibraryGroup::GetItems() const
{
	return Items;
}

int32 FMediaViewerLibraryGroup::AddItem(const FGuid& InItemId, int32 InIndex)
{
	if (bDynamic)
	{
		return false;
	}

	if (InIndex == INDEX_NONE || InIndex >= Items.Num())
	{
		return Items.Add(InItemId);
	}

	Items.Insert(InItemId, InIndex);
	return InIndex;
}

int32 FMediaViewerLibraryGroup::FindItemIndex(const FGuid& InItemId) const
{
	for (int32 Index = 0; Index < Items.Num(); ++Index)
	{
		if (Items[Index] == InItemId)
		{
			return Index;
		}
	}

	return INDEX_NONE;
}

bool FMediaViewerLibraryGroup::ContainsItem(const FGuid& InItemId) const
{
	return FindItemIndex(InItemId) != INDEX_NONE;
}

bool FMediaViewerLibraryGroup::RemoveItem(const FGuid& InItemId)
{
	if (bDynamic)
	{
		return false;
	}

	return RemoveItemAt(FindItemIndex(InItemId));
}

bool FMediaViewerLibraryGroup::RemoveItemAt(int32 InIndex)
{
	if (bDynamic)
	{
		return false;
	}

	if (!Items.IsValidIndex(InIndex))
	{
		return false;
	}

	Items.RemoveAt(InIndex);
	return true;
}

int32 FMediaViewerLibraryGroup::Empty()
{
	if (!bDynamic)
	{
		const int32 Count = Items.Num();
		Items.Empty();
		return Count;
	}

	return 0;
}

bool FMediaViewerLibraryGroup::IsDynamic() const
{
	return bDynamic;
}

#undef LOCTEXT_NAMESPACE
