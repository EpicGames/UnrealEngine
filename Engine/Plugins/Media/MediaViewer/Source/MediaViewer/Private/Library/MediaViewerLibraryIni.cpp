// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/MediaViewerLibraryIni.h"

#include "IMediaViewerModule.h"
#include "Library/MediaViewerLibrary.h"
#include "Library/MediaViewerLibraryGroup.h"
#include "Library/MediaViewerLibraryItem.h"
#include "Templates/SharedPointer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaViewerLibraryIni)

UMediaViewerLibraryIni& UMediaViewerLibraryIni::Get()
{
	return *GetMutableDefault<UMediaViewerLibraryIni>();
}

UMediaViewerLibraryIni::UMediaViewerLibraryIni()
{
	Bookmarks.SetNum(10);
}

void UMediaViewerLibraryIni::SaveLibrary(const TSharedRef<UE::MediaViewer::IMediaViewerLibrary>& InLibrary)
{
	using namespace UE::MediaViewer::Private;

	TSharedRef<FMediaViewerLibrary> LibraryImpl = StaticCastSharedRef<FMediaViewerLibrary>(InLibrary);

	Groups.Empty();
	Groups.Reserve(LibraryImpl->Groups.Num());

	for (const TSharedRef<FMediaViewerLibraryGroup>& Group : LibraryImpl->Groups)
	{
		if (Group->IsDynamic())
		{
			continue;
		}

		Groups.Add(*Group);
	}

	Items.Empty();
	Items.Reserve(LibraryImpl->Items.Num());

	for (const TPair<FGuid, TSharedRef<FMediaViewerLibraryItem>>& ItemPair : LibraryImpl->Items)
	{
		const TSharedRef<FMediaViewerLibraryItem>& Item = ItemPair.Value;

		if (Item->IsTransient())
		{
			continue;
		}

		Items.Emplace(Item->GetItemType(), *Item);
	}
}

void UMediaViewerLibraryIni::LoadLibrary(const TSharedRef<UE::MediaViewer::IMediaViewerLibrary>& InLibrary) const
{
	using namespace UE::MediaViewer::Private;

	TSharedRef<FMediaViewerLibrary> LibraryImpl = StaticCastSharedRef<FMediaViewerLibrary>(InLibrary);

	UE::MediaViewer::IMediaViewerModule& Module = UE::MediaViewer::IMediaViewerModule::Get();

	for (const FMediaViewerLibraryItemData& ItemData : Items)
	{
		const FGuid ItemId = ItemData.Item.GetId();

		if (!LibraryImpl->Items.Contains(ItemId))
		{
			if (TSharedPtr<FMediaViewerLibraryItem> Item = Module.CreateLibraryItem(ItemData.ItemType, ItemData.Item))
			{
				LibraryImpl->Items.Add(ItemId, Item.ToSharedRef());
			}
		}
	}

	for (const FMediaViewerLibraryGroup& SavedGroup : Groups)
	{
		TSharedPtr<FMediaViewerLibraryGroup> Group = LibraryImpl->GetGroup(SavedGroup.GetId());

		if (!Group.IsValid())
		{
			Group = MakeShared<FMediaViewerLibraryGroup>(FMediaViewerLibraryGroup::FPrivateToken(), SavedGroup);
			LibraryImpl->AddGroup(Group.ToSharedRef());
		}

		TSet<FGuid> ExistingItems;
		ExistingItems.Append(Group->Items);

		for (const FGuid& ItemId : SavedGroup.Items)
		{
			if (!ExistingItems.Contains(ItemId))
			{
				Group->Items.Add(ItemId);
				ExistingItems.Add(ItemId);
			}
		}

		LibraryImpl->RemoveInvalidGroupItems(SavedGroup.GetId());
	}
}

bool UMediaViewerLibraryIni::HasGroup(const FGuid& InGroupId) const
{
	for (const FMediaViewerLibraryGroup& Group : Groups)
	{
		if (Group.GetId() == InGroupId)
		{
			return true;
		}
	}

	return false;
}

bool UMediaViewerLibraryIni::HasItem(const FGuid& InItemId) const
{
	for (const FMediaViewerLibraryItemData& ItemData : Items)
	{
		if (ItemData.Item.GetId() == InItemId)
		{
			return true;
		}
	}

	return false;
}

TConstArrayView<FMediaViewerState> UMediaViewerLibraryIni::GetBookmarks() const
{
	return Bookmarks;
}

void UMediaViewerLibraryIni::SetBookmark(int32 InIndex, const FMediaViewerState& InState)
{
	if (Bookmarks.IsValidIndex(InIndex))
	{
		Bookmarks[InIndex] = InState;
	}
}

TOptional<FMediaViewerState> UMediaViewerLibraryIni::GetLastOpenedState() const
{
	return LastOpenedState;
}

void UMediaViewerLibraryIni::SetLastOpenedState(const FMediaViewerState& InState)
{
	LastOpenedState = InState;
}
