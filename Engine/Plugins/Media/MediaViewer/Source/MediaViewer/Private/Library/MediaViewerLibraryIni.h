// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "Library/MediaViewerLibraryGroup.h"
#include "Library/MediaViewerLibraryItem.h"
#include "ImageViewer/MediaImageViewer.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Widgets/MediaViewerSettings.h"
#include "Widgets/SMediaViewer.h"

#include "MediaViewerLibraryIni.generated.h"

namespace UE::MediaViewer
{
	class IMediaViewerLibrary;
}

USTRUCT()
struct FMediaViewerLibraryItemData
{
	GENERATED_BODY()

	UPROPERTY()
	FName ItemType;

	UPROPERTY()
	FMediaViewerLibraryItem Item;
};

USTRUCT()
struct FMediaViewerImageState
{
	GENERATED_BODY()

	UPROPERTY(Config)
	FName ImageType;

	UPROPERTY(Config)
	FString StringValue;

	UPROPERTY(Config)
	FMediaImageViewerPanelSettings PanelSettings;

	UPROPERTY(Config)
	FMediaImagePaintSettings PaintSettings;
};

USTRUCT()
struct FMediaViewerState
{
	GENERATED_BODY()

	UPROPERTY(Config)
	FMediaViewerSettings ViewerSettings;

	UPROPERTY(Config)
	EMediaImageViewerActivePosition ActiveView = EMediaImageViewerActivePosition::Single;

	UPROPERTY(Config)
	TArray<FMediaViewerImageState> Images;
};

UCLASS(Config = EditorPerProjectUserSettings)
class UMediaViewerLibraryIni : public UObject
{
	GENERATED_BODY()

public:
	static UMediaViewerLibraryIni& Get();

	UMediaViewerLibraryIni();

	void SaveLibrary(const TSharedRef<UE::MediaViewer::IMediaViewerLibrary>& InLibrary);

	void LoadLibrary(const TSharedRef<UE::MediaViewer::IMediaViewerLibrary>& InLibrary) const;

	bool HasGroup(const FGuid& InGroupId) const;

	bool HasItem(const FGuid& InItemId) const;

	TConstArrayView<FMediaViewerState> GetBookmarks() const;

	void SetBookmark(int32 InIndex, const FMediaViewerState& InState);

	TOptional<FMediaViewerState> GetLastOpenedState() const;

	void SetLastOpenedState(const FMediaViewerState& InState);

protected:
	UPROPERTY(Config)
	TArray<FMediaViewerLibraryGroup> Groups;

	UPROPERTY(Config)
	TArray<FMediaViewerLibraryItemData> Items;

	UPROPERTY(Config)
	TArray<FMediaViewerState> Bookmarks;

	UPROPERTY(Config)
	TOptional<FMediaViewerState> LastOpenedState;
};
