// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/MediaViewerLibraryItem.h"

#include "AssetDefinitionRegistry.h"
#include "AssetToolsModule.h"
#include "Modules/ModuleManager.h"
#include "Styling/StyleColors.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaViewerLibraryItem)

#define LOCTEXT_NAMESPACE "MediaViewerLibraryItem"

FMediaViewerLibraryItem::FMediaViewerLibraryItem()
	: FMediaViewerLibraryItem(FText::GetEmpty(), FText::GetEmpty(), /* Transient */ false, TEXT(""))
{
}

FMediaViewerLibraryItem::FMediaViewerLibraryItem(const FText& InName, const FText& InToolTip,
	bool bInTransient, const FString& InStringValue)
	: FMediaViewerLibraryItem(FGuid::NewGuid(), InName, InToolTip, bInTransient, InStringValue)
{
}

FMediaViewerLibraryItem::FMediaViewerLibraryItem(const FGuid& InId, const FText& InName, const FText& InToolTip, 
	bool bInTransient, const FString& InStringValue)
	: FMediaViewerLibraryEntry(InId, InName, InToolTip)
	, bTransient(bInTransient)
	, StringValue(InStringValue)
{
}

FName FMediaViewerLibraryItem::GetItemType() const
{
	return NAME_None;
}

FText FMediaViewerLibraryItem::GetItemTypeDisplayName() const
{
	return LOCTEXT("Error", "Error");
}

FSlateColor FMediaViewerLibraryItem::GetItemTypeColor() const
{
	return GetClassColor(nullptr);
}

bool FMediaViewerLibraryItem::IsTransient() const
{
	return bTransient;
}

const FString& FMediaViewerLibraryItem::GetStringValue() const
{
	return StringValue;
}

FSlateColor FMediaViewerLibraryItem::GetClassColor(UClass* InClass)
{
	if (InClass)
	{
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
		TWeakPtr<IAssetTypeActions> AssetTypeActionsWeak = AssetToolsModule.Get().GetAssetTypeActionsForClass(InClass);

		if (TSharedPtr<IAssetTypeActions> AssetTypeActions = AssetTypeActionsWeak.Pin())
		{
			return AssetTypeActions->GetTypeColor();
		}
	}

	return FSlateColor::UseForeground();
}

#undef LOCTEXT_NAMESPACE
