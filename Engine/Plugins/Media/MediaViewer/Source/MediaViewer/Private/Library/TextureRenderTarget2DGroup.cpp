// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/TextureRenderTarget2DGroup.h"

#include "Engine/TextureRenderTarget2D.h"
#include "ImageViewers/TextureRenderTarget2DImageViewer.h"
#include "Library/MediaViewerLibraryItem.h"
#include "MediaViewerUtils.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "TextureRenderTarget2DGroup"

namespace UE::MediaViewer::Private
{

FTextureRenderTarget2DGroup::FTextureRenderTarget2DGroup(const TSharedRef<IMediaViewerLibrary>& InLibrary)
	: FTextureRenderTarget2DGroup(InLibrary, FGuid::NewGuid())
{
}

FTextureRenderTarget2DGroup::FTextureRenderTarget2DGroup(const TSharedRef<IMediaViewerLibrary>& InLibrary, 
	const FGuid& InGuid)
	: FMediaViewerLibraryDynamicGroup(
		InLibrary,
		InGuid,
		LOCTEXT("RenderTargets", "Render Targets"),
		LOCTEXT("RenderTargetsTooltip", "The available render targets."),
		FGenerateItems::CreateStatic(&FTextureRenderTarget2DGroup::GetTextureRenderTarget2DItems)
	)
{
}

TArray<TSharedRef<FMediaViewerLibraryItem>> FTextureRenderTarget2DGroup::GetTextureRenderTarget2DItems()
{
	TArray<TSharedRef<FMediaViewerLibraryItem>> TextureRenderTarget2DItems;

	TSharedRef<FTextureRenderTarget2DImageViewer::FFactory> Factory = MakeShared<FTextureRenderTarget2DImageViewer::FFactory>();

	for (UTextureRenderTarget2D* TextureRenderTarget2D : TObjectRange<UTextureRenderTarget2D>())
	{
		if (TextureRenderTarget2D->IsTemplate())
		{
			continue;
		}

		if (TextureRenderTarget2D->HasAssetUserDataOfClass(UMediaViewerUserData::StaticClass()))
		{
			continue;
		}

		if (TSharedPtr<FMediaViewerLibraryItem> NewItem = Factory->CreateLibraryItem(TNotNull<UTextureRenderTarget2D*>(TextureRenderTarget2D)))
		{
			TextureRenderTarget2DItems.Add(NewItem.ToSharedRef());
		}
	}

	return TextureRenderTarget2DItems;
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
