// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Library/MediaViewerLibraryDynamicGroup.h"

namespace UE::MediaViewer::Private
{

/**
 * A group that generates entries based on available UTextureRenderTarget2Ds.
*/
struct FTextureRenderTarget2DGroup : public FMediaViewerLibraryDynamicGroup
{
	FTextureRenderTarget2DGroup(const TSharedRef<IMediaViewerLibrary>& InLibrary);
	FTextureRenderTarget2DGroup(const TSharedRef<IMediaViewerLibrary>& InLibrary, const FGuid& InId);

	virtual ~FTextureRenderTarget2DGroup() override = default;

protected:
	static TArray<TSharedRef<FMediaViewerLibraryItem>> GetTextureRenderTarget2DItems();
};

} // UE::MediaViewer::Private
