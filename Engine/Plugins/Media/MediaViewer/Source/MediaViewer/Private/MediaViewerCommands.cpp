// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaViewerCommands.h"

#include "Framework/Commands/InputChord.h"
#include "MediaViewerStyle.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "MediaViewerCommands"

namespace UE::MediaViewer::Private
{

FMediaViewerCommands::FMediaViewerCommands()
	: TCommands("MediaViewer", LOCTEXT("ContextDescription", "Media Viewer"), NAME_None, FMediaViewerStyle::Get().GetStyleSetName())
{
}

void FMediaViewerCommands::RegisterCommands()
{
	UI_COMMAND(
		ToggleOverlay, 
		"Show Overlays", 
		"Toggles the visibility of the status bar and any custom overlays.", 
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::O)
	);

	UI_COMMAND(
		MoveLeft,
		"Move Left",
		"Moves the camera focus point to the left.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::A),
		FInputChord(EKeys::Left)
	);

	UI_COMMAND(
		MoveRight,
		"Move Right",
		"Moves the camera focus point to the right.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::D),
		FInputChord(EKeys::Right)
	);

	UI_COMMAND(
		MoveUp,
		"Move Up",
		"Moves the camera focus point to the up.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::W),
		FInputChord(EKeys::Up)
	);

	UI_COMMAND(
		MoveDown,
		"Move Down",
		"Moves the camera focus point to the down.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::S),
		FInputChord(EKeys::Down)
	);

	UI_COMMAND(
		MoveForward,
		"Move Forward",
		"Moves the camera focus point to the forward.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Q),
		FInputChord(EKeys::PageUp)
	);

	UI_COMMAND(
		MoveBackward,
		"Move Backward",
		"Moves the camera focus point to the backward.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::E),
		FInputChord(EKeys::PageDown)
	);

	UI_COMMAND(
		RotatePlusYaw,
		"Rotate +Yaw",
		"Rotates the camera around the Z axis.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::NumPadSix)
	);

	UI_COMMAND(
		RotateMinusYaw,
		"Rotate -Yaw",
		"Rotates the camera around the Z axis.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::NumPadFour)
	);

	UI_COMMAND(
		RotatePlusPitch,
		"Rotate +Pitch",
		"Rotates the camera around the X axis.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::NumPadEight)
	);

	UI_COMMAND(
		RotateMinusPitch,
		"Rotate -Pitch",
		"Rotates the camera around the X axis.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::NumPadTwo)
	);

	UI_COMMAND(
		RotatePlusRoll,
		"Rotate +Roll",
		"Rotates the camera around the Y axis.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::NumPadThree)
	);

	UI_COMMAND(
		RotateMinusRoll,
		"Rotate -Roll",
		"Rotates the camera around the Y axis.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::NumPadOne)
	);

	UI_COMMAND(
		ScaleUp, 
		"Zoom In", 
		"Zoom in on the image.", 
		EUserInterfaceActionType::Button, 
		FInputChord(EKeys::Equals)
	);

	UI_COMMAND(
		ScaleDown,
		"Zoom Out",
		"Zoom out of the image.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Hyphen)
	);

	UI_COMMAND(
		ToggleAutoFiltering,
		"Toggle Auto Filtering",
		"Toggles automatic filtering. When enabled, bilinear filtering is disabled once the zoom level reaches the configured threshold, allowing individual pixels to be seen clearly.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::I)
	);

	UI_COMMAND(
		TogglePixelGrid,
		"Display Pixel Grid",
		"Toggles the pixel grid overlay. When enabled, a grid is drawn over the image at high zoom levels to make individual pixel boundaries clearly visible.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::G)
	);

	UI_COMMAND(
		ToggleMipOverride,
		"Toggle Mip Override",
		"Toggles whether to override the mip level of the image.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::M)
	);

	UI_COMMAND(
		IncreaseMipLevel,
		"Increase Mip Level",
		"Increase the Mip level by 1 (decreasing quality).",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Period)
	);

	UI_COMMAND(
		DecreaseMipLevel,
		"Decrease Mip Level",
		"Decrease the Mip level by 1 (increasing quality).",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Comma)
	);

	UI_COMMAND(
		ScaleToFit, 
		"Scale to Fit", 
		"Changes the scale of the image so it matches, but does not exceed, the size of the viewport.", 
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::F)
	);

	UI_COMMAND(
		Scale12,
		"12.5%",
		"Set Scale to 12.5%.",
		EUserInterfaceActionType::RadioButton,
		FInputChord()
	);

	UI_COMMAND(
		Scale25,
		"25%",
		"Set Scale to 25%.",
		EUserInterfaceActionType::RadioButton,
		FInputChord()
	);

	UI_COMMAND(
		Scale50,
		"50%",
		"Set Scale to 50%.",
		EUserInterfaceActionType::RadioButton,
		FInputChord()
	);

	UI_COMMAND(
		Scale100,
		"100%",
		"Set Scale to 100%.",
		EUserInterfaceActionType::RadioButton,
		FInputChord()
	);

	UI_COMMAND(
		Scale200,
		"200%",
		"Set Scale to 200%.",
		EUserInterfaceActionType::RadioButton,
		FInputChord()
	);

	UI_COMMAND(
		Scale400,
		"400%",
		"Set Scale to 400%.",
		EUserInterfaceActionType::RadioButton,
		FInputChord()
	);

	UI_COMMAND(
		Scale800,
		"800%",
		"Set Scale to 800%.",
		EUserInterfaceActionType::RadioButton,
		FInputChord()
	);

	UI_COMMAND(
		Scale1000,
		"1000%",
		"Set Scale to 1000%.",
		EUserInterfaceActionType::RadioButton,
		FInputChord()
	);

	UI_COMMAND(
		CopyTransform,
		"Copy Transform",
		"Copiess the image transform to the other viewer.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Insert)
	);

	UI_COMMAND(
		ResetTransform,
		"Reset Transform",
		"Resets the image transform.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Home),
		FInputChord(EKeys::NumPadZero)
	);

	UI_COMMAND(
		ResetAllTransforms,
		"Reset All Transforms",
		"Resets all the image transforms.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Delete),
		FInputChord(EKeys::Decimal)
	);

	UI_COMMAND(
		ToggleLockedTransform,
		"Sync Transforms",
		"Syncs the transforms of all images so that changes made to one are made to all of them.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::T)
	);

	UI_COMMAND(
		MirrorHorizonal,
		"Mirror Horizontally",
		"Mirror the image on the horizontal axis.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::H)
	);

	UI_COMMAND(
		MirrorVertical,
		"Mirror Vertically",
		"Mirror the image on the vertical axis.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::V)
	);

	UI_COMMAND(
		MirrorDepth,
		"Mirror Depth",
		"Mirror the image on the depth axis.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::B)
	);

	UI_COMMAND(
		SecondImageOpacityMinus,
		"Lower B Image Opacity",
		"Make the B Image more translucent.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::LeftBracket)
	);

	UI_COMMAND(
		SecondImageOpacityPlus,
		"Higher B Image Opacity",
		"Make the B Image more opaque.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::RightBracket)
	);

	UI_COMMAND(
		CopyColor,
		"Copy Color",
		"Copy the color under the crosshair to the clipboard.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::C)
	);

	UI_COMMAND(
		AddToLibrary,
		"Pin Image",
		"Add this image to the Pinned group in the Library.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::S, EModifierKey::Control)
	);

	UI_COMMAND(
		SwapAB,
		"Swap A and B",
		"Swap A and B images and their transforms.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Slash)
	);

	UI_COMMAND(
		Snapshot,
		"Snapshot",
		"Take a snapshot of the current viewer display.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::S, EModifierKey::Alt)
	);

	UI_COMMAND(
		QuickRewindVideo,
		"Quick Rewind Video",
		"Jumps back 10s in the currently active video.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::J, EModifierKey::Shift)
	);

	UI_COMMAND(
		RewindVideo,
		"Rewind Video",
		"Jumps back 1s in the currently active video.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::J)
	);

	UI_COMMAND(
		StepBackVideo,
		"Step Back Video",
		"Jumps back 1 frame in the currently active video.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::J, EModifierKey::Control),
		FInputChord(EKeys::J, EModifierKey::Command)
	);

	UI_COMMAND(
		ToggleVideoPlay,
		"Toggle Play/Pause",
		"Toggles between play and pause on the current video.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::K),
		FInputChord(EKeys::SpaceBar)
	);

	UI_COMMAND(
		StepForwardVideo,
		"Step Forward Video",
		"Jumps forward 1 frame in the currently active video.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::L, EModifierKey::Control),
		FInputChord(EKeys::L, EModifierKey::Command)
	);

	UI_COMMAND(
		FastForwardVideo,
		"Fast Forward Video",
		"Jumps forward 1s in the currently active video.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::L)
	);

	UI_COMMAND(
		QuickFastForwardVideo,
		"Quick Fast Forward Video",
		"Jumps forward 10s in the currently active video.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::L, EModifierKey::Shift)
	);

	UI_COMMAND(
		OpenNewTab,
		"New Tab",
		"Open a new standalone tab.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::N, EModifierKey::Control),
		FInputChord(EKeys::N, EModifierKey::Command)
	);

	UI_COMMAND(
		OpenBookmark1,
		"Bookmark 1",
		"Restore bookmark 1.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::One)
	);

	UI_COMMAND(
		OpenBookmark2,
		"Bookmark 2",
		"Restore bookmark 2.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Two)
	);

	UI_COMMAND(
		OpenBookmark3,
		"Bookmark 3",
		"Restore bookmark 3.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Three)
	);

	UI_COMMAND(
		OpenBookmark4,
		"Bookmark 4",
		"Restore bookmark 4.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Four)
	);

	UI_COMMAND(
		OpenBookmark5,
		"Bookmark 5",
		"Restore bookmark 5.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Five)
	);

	UI_COMMAND(
		OpenBookmark6,
		"Bookmark 6",
		"Restore bookmark 6.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Six)
	);

	UI_COMMAND(
		OpenBookmark7,
		"Bookmark 7",
		"Restore bookmark 7.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Seven)
	);

	UI_COMMAND(
		OpenBookmark8,
		"Bookmark 8",
		"Restore bookmark 8.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Eight)
	);

	UI_COMMAND(
		OpenBookmark9,
		"Bookmark 9",
		"Restore bookmark 9.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Nine)
	);

	UI_COMMAND(
		OpenBookmark10,
		"Bookmark 10",
		"Restore bookmark 10.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Zero)
	);

	UI_COMMAND(
		SaveBookmark1,
		"Save Bookmark 1",
		"Save bookmark 1.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::One, EModifierKey::Control)
	);

	UI_COMMAND(
		SaveBookmark2,
		"Save Bookmark 2",
		"Save bookmark 2.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Two, EModifierKey::Control)
	);

	UI_COMMAND(
		SaveBookmark3,
		"Save Bookmark 3",
		"Save bookmark 3.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Three, EModifierKey::Control)
	);

	UI_COMMAND(
		SaveBookmark4,
		"Save Bookmark 4",
		"Save bookmark 4.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Four, EModifierKey::Control)
	);

	UI_COMMAND(
		SaveBookmark5,
		"Save Bookmark 5",
		"Save bookmark 5.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Five, EModifierKey::Control)
	);

	UI_COMMAND(
		SaveBookmark6,
		"Save Bookmark 6",
		"Save bookmark 6.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Six, EModifierKey::Control)
	);

	UI_COMMAND(
		SaveBookmark7,
		"Save Bookmark 7",
		"Save bookmark 7.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Seven, EModifierKey::Control)
	);

	UI_COMMAND(
		SaveBookmark8,
		"Save Bookmark 8",
		"Save bookmark 8.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Eight, EModifierKey::Control)
	);

	UI_COMMAND(
		SaveBookmark9,
		"Save Bookmark 9",
		"Save bookmark 9.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Nine, EModifierKey::Control)
	);

	UI_COMMAND(
		SaveBookmark10,
		"Save Bookmark 10",
		"Save bookmark 10.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Zero, EModifierKey::Control)
	);
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
