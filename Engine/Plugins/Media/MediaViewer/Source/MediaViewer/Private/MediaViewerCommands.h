// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

#include "Framework/Commands/UICommandInfo.h"

namespace UE::MediaViewer::Private
{

class FMediaViewerCommands : public TCommands<FMediaViewerCommands>
{
public:
	FMediaViewerCommands();

	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> ToggleOverlay;

	TSharedPtr<FUICommandInfo> MoveLeft;
	TSharedPtr<FUICommandInfo> MoveRight;
	TSharedPtr<FUICommandInfo> MoveUp;
	TSharedPtr<FUICommandInfo> MoveDown;
	TSharedPtr<FUICommandInfo> MoveForward;
	TSharedPtr<FUICommandInfo> MoveBackward;

	TSharedPtr<FUICommandInfo> RotatePlusYaw;
	TSharedPtr<FUICommandInfo> RotateMinusYaw;
	TSharedPtr<FUICommandInfo> RotatePlusPitch;
	TSharedPtr<FUICommandInfo> RotateMinusPitch;
	TSharedPtr<FUICommandInfo> RotatePlusRoll;
	TSharedPtr<FUICommandInfo> RotateMinusRoll;
	
	TSharedPtr<FUICommandInfo> ScaleUp;
	TSharedPtr<FUICommandInfo> ScaleDown;

	TSharedPtr<FUICommandInfo> ToggleAutoFiltering;
	TSharedPtr<FUICommandInfo> TogglePixelGrid;

	TSharedPtr<FUICommandInfo> ToggleMipOverride;
	TSharedPtr<FUICommandInfo> IncreaseMipLevel;
	TSharedPtr<FUICommandInfo> DecreaseMipLevel;

	TSharedPtr<FUICommandInfo> Scale12;
	TSharedPtr<FUICommandInfo> Scale25;
	TSharedPtr<FUICommandInfo> Scale50;
	TSharedPtr<FUICommandInfo> Scale100;
	TSharedPtr<FUICommandInfo> Scale200;
	TSharedPtr<FUICommandInfo> Scale400;
	TSharedPtr<FUICommandInfo> Scale800;
	TSharedPtr<FUICommandInfo> Scale1000;
	TSharedPtr<FUICommandInfo> ScaleToFit;

	TSharedPtr<FUICommandInfo> CopyTransform;
	TSharedPtr<FUICommandInfo> ResetTransform;
	TSharedPtr<FUICommandInfo> ResetAllTransforms;
	TSharedPtr<FUICommandInfo> ToggleLockedTransform;

	TSharedPtr<FUICommandInfo> MirrorHorizonal;
	TSharedPtr<FUICommandInfo> MirrorVertical;
	TSharedPtr<FUICommandInfo> MirrorDepth;

	TSharedPtr<FUICommandInfo> SecondImageOpacityMinus;
	TSharedPtr<FUICommandInfo> SecondImageOpacityPlus;

	TSharedPtr<FUICommandInfo> CopyColor;

	TSharedPtr<FUICommandInfo> AddToLibrary;

	TSharedPtr<FUICommandInfo> SwapAB;

	TSharedPtr<FUICommandInfo> Snapshot;

	TSharedPtr<FUICommandInfo> QuickRewindVideo;
	TSharedPtr<FUICommandInfo> RewindVideo;
	TSharedPtr<FUICommandInfo> StepBackVideo;
	TSharedPtr<FUICommandInfo> ToggleVideoPlay;
	TSharedPtr<FUICommandInfo> StepForwardVideo;
	TSharedPtr<FUICommandInfo> FastForwardVideo;
	TSharedPtr<FUICommandInfo> QuickFastForwardVideo;

	TSharedPtr<FUICommandInfo> OpenNewTab;

	TSharedPtr<FUICommandInfo> OpenBookmark1;
	TSharedPtr<FUICommandInfo> OpenBookmark2;
	TSharedPtr<FUICommandInfo> OpenBookmark3;
	TSharedPtr<FUICommandInfo> OpenBookmark4;
	TSharedPtr<FUICommandInfo> OpenBookmark5;
	TSharedPtr<FUICommandInfo> OpenBookmark6;
	TSharedPtr<FUICommandInfo> OpenBookmark7;
	TSharedPtr<FUICommandInfo> OpenBookmark8;
	TSharedPtr<FUICommandInfo> OpenBookmark9;
	TSharedPtr<FUICommandInfo> OpenBookmark10;

	TSharedPtr<FUICommandInfo> SaveBookmark1;
	TSharedPtr<FUICommandInfo> SaveBookmark2;
	TSharedPtr<FUICommandInfo> SaveBookmark3;
	TSharedPtr<FUICommandInfo> SaveBookmark4;
	TSharedPtr<FUICommandInfo> SaveBookmark5;
	TSharedPtr<FUICommandInfo> SaveBookmark6;
	TSharedPtr<FUICommandInfo> SaveBookmark7;
	TSharedPtr<FUICommandInfo> SaveBookmark8;
	TSharedPtr<FUICommandInfo> SaveBookmark9;
	TSharedPtr<FUICommandInfo> SaveBookmark10;
};

} // UE::MediaViewer::Private
