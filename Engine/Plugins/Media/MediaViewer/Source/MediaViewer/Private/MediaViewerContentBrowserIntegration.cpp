// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaViewerContentBrowserIntegration.h"

#include "AssetRegistry/AssetData.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserModule.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Internationalization/Text.h"
#include "MediaViewer.h"
#include "MediaViewerModule.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "UObject/Class.h"

#define LOCTEXT_NAMESPACE "MediaViewerContentBrowserIntegration"

namespace UE::MediaViewer
{

void FMediaViewerContentBrowserIntegration::Integrate()
{
	Disintegrate();

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	TArray<FContentBrowserMenuExtender_SelectedAssets>& CBMenuExtenderDelegates = ContentBrowserModule.GetAllAssetViewContextMenuExtenders();
	CBMenuExtenderDelegates.Add(FContentBrowserMenuExtender_SelectedAssets::CreateSP(this, &FMediaViewerContentBrowserIntegration::OnExtendContentBrowserAssetSelectionMenu));
	ContentBrowserAssetHandle = CBMenuExtenderDelegates.Last().GetHandle();
}

void FMediaViewerContentBrowserIntegration::Disintegrate()
{
	if (ContentBrowserAssetHandle.IsValid())
	{
		if (FContentBrowserModule* ContentBrowserModule = FModuleManager::GetModulePtr<FContentBrowserModule>("ContentBrowser"))
		{
			TArray<FContentBrowserMenuExtender_SelectedAssets>& CBMenuExtenderDelegates = ContentBrowserModule->GetAllAssetViewContextMenuExtenders();

			CBMenuExtenderDelegates.RemoveAll(
				[this](const FContentBrowserMenuExtender_SelectedAssets& InElement)
				{
					return InElement.GetHandle() == ContentBrowserAssetHandle;
				});

			ContentBrowserAssetHandle.Reset();
		}
	}
}

TSharedRef<FExtender> FMediaViewerContentBrowserIntegration::OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& InSelectedAssets)
{
	using namespace UE::MediaViewer;

	const IMediaViewerModule& MediaViewerModule = IMediaViewerModule::Get();

	TSharedRef<FExtender> Extender = MakeShared<FExtender>();
	int32 AssetCount = 0;

	for (const FAssetData& SelectedAsset : InSelectedAssets)
	{
		if (MediaViewerModule.HasFactoryFor(SelectedAsset))
		{
			++AssetCount;
		}
	}

	if (AssetCount == 0)
	{
		return Extender;
	}

	Extender->AddMenuExtension(
		"GetAssetActions",
		EExtensionHook::After,
		nullptr,
		FMenuExtensionDelegate::CreateLambda(
			[InSelectedAssets, AssetCount](FMenuBuilder& InMenuBuilder)
			{
				const FText ToolTip = AssetCount == 1
					? LOCTEXT("OpenInMediaViewerSingleTooltip", "Open this asset in the Media Viewer, replacing the Single View or A image.")
					: LOCTEXT("OpenInMediaViewerMultiTooltip", "Open the first 2 valid assets in the Media Viewer in the A/B view.");

				InMenuBuilder.AddMenuEntry(
					LOCTEXT("OpenInMediaViewer", "Open in Media Viewer"),
					ToolTip,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.Tracks.Media"),
					FUIAction(FExecuteAction::CreateStatic(&FMediaViewerContentBrowserIntegration::OpenInMediaViewer, InSelectedAssets))
				);
			}
		)
	);

	return Extender;
}

void FMediaViewerContentBrowserIntegration::OpenInMediaViewer(TArray<FAssetData> InSelectedAssets)
{
	IMediaViewerModule& MediaViewerModule = IMediaViewerModule::Get();

	if (!MediaViewerModule.OpenTab())
	{
		UE_LOGF(LogMediaViewer, Error, "Unable to open Media Viewer Tab.");
		return;
	}

	bool bHasOpenedAsset = false;

	for (const FAssetData& SelectedAsset : InSelectedAssets)
	{
		if (MediaViewerModule.HasFactoryFor(SelectedAsset))
		{
			if (!bHasOpenedAsset)
			{
				MediaViewerModule.SetImage(EMediaImageViewerPosition::First, SelectedAsset);
				bHasOpenedAsset = true;
			}
			else
			{
				MediaViewerModule.SetImage(EMediaImageViewerPosition::Second, SelectedAsset);
				return;
			}
		}
	}
}

} // UE::MediaViewer

#undef LOCTEXT_NAMESPACE
