// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#include "Containers/ContainersFwd.h"
#include "Delegates/IDelegateInstance.h"

class FExtender;
struct FAssetData;

namespace UE::MediaViewer
{

class FMediaViewerContentBrowserIntegration : public TSharedFromThis<FMediaViewerContentBrowserIntegration>
{
public:
	static const TSharedRef<FMediaViewerContentBrowserIntegration>& Get()
	{
		static TSharedRef<FMediaViewerContentBrowserIntegration> Integration = MakeShared<FMediaViewerContentBrowserIntegration>();
		return Integration;
	}

	void Integrate();

	void Disintegrate();

protected:
	static void OpenInMediaViewer(TArray<FAssetData> InSelectedAssets);

	FDelegateHandle ContentBrowserAssetHandle;

	TSharedRef<FExtender> OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& InSelectedAssets);
};

} // UE::MediaViewer
