// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaViewerModule.h"

#include "AssetRegistry/AssetData.h"
#include "Framework/Docking/TabManager.h"
#include "ImageViewers/ColorImageViewer.h"
#include "ImageViewers/MaterialInterfaceImageViewer.h"
#include "ImageViewers/MediaSourceImageViewer.h"
#include "ImageViewers/MediaTextureImageViewer.h"
#include "ImageViewers/NullImageViewer.h"
#include "ImageViewers/Texture2DImageViewer.h"
#include "ImageViewers/TextureRenderTarget2DImageViewer.h"
#include "LevelEditor.h"
#include "Library/LevelEditorViewportGroup.h"
#include "Library/MediaTextureGroup.h"
#include "Library/MediaTrackGroup.h"
#include "Library/MediaViewerLibrary.h"
#include "Library/MediaViewerLibraryIni.h"
#include "MediaViewer.h"
#include "MediaViewerCommands.h"
#include "MediaViewerContentBrowserIntegration.h"
#include "MediaViewerDelegates.h"
#include "MediaViewerStyle.h"
#include "MediaViewerUtils.h"
#include "Misc/NotNull.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SMediaViewer.h"
#include "Widgets/SMediaViewerLibrary.h"
#include "Widgets/SMediaViewerTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

DEFINE_LOG_CATEGORY(LogMediaViewer);

#define LOCTEXT_NAMESPACE "MediaViewerModule"

namespace UE::MediaViewer::Private
{

TSharedPtr<FTabManager> GetLevelEditorTabManager()
{
	if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
	{
		if (TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule->GetFirstLevelEditor())
		{
			return LevelEditor->GetTabManager();
		}
	}

	return nullptr;
}

const FLazyName FMediaViewerModule::TabId = "MediaViewerTabId";
const FLazyName FMediaViewerModule::StandaloneTabId = "MediaViewerStandaloneTabId";

FMediaViewerModule::FMediaViewerModule()
	: Library(MakeShared<FMediaViewerLibrary>())
{
	static FDelayedAutoRegisterHelper LoadLibraryCallback(
		EDelayedRegisterRunPhase::EndOfEngineInit,
		[]
		{
			TSharedRef<FMediaViewerLibrary> LibraryImpl = StaticCastSharedRef<FMediaViewerLibrary>(IMediaViewerModule::Get().GetLibrary());

			UMediaViewerLibraryIni::Get().LoadLibrary(LibraryImpl);

			// Add dynamic groups
			LibraryImpl->AddGroup(MakeShared<FLevelEditorViewportGroup>(LibraryImpl));
			LibraryImpl->AddGroup(MakeShared<FMediaTrackGroup>(LibraryImpl));
			LibraryImpl->AddGroup(MakeShared<FMediaTextureGroup>(LibraryImpl));
		}
	);
}

bool FMediaViewerModule::IsFactoryRegistered(FName InFactoryName) const
{
	return Factories.Contains(InFactoryName);
}

bool FMediaViewerModule::HasFactoryFor(const FAssetData& InAssetData) const
{
	for (const TPair<FName, TSharedRef<IMediaImageViewerFactory>>& FactoryPair : Factories)
	{
		if (FactoryPair.Value->SupportsAsset(InAssetData))
		{
			return true;
		}
	}

	return false;
}

bool FMediaViewerModule::HasFactoryFor(UObject* InObject) const
{
	for (const TPair<FName, TSharedRef<IMediaImageViewerFactory>>& FactoryPair : Factories)
	{
		if (FactoryPair.Value->SupportsAsset(InObject))
		{
			return true;
		}
	}

	return false;
}

void FMediaViewerModule::RegisterFactory(FName InFactoryName, const TSharedRef<IMediaImageViewerFactory>& InFactory)
{
	if (Factories.Contains(InFactoryName))
	{
		Factories[InFactoryName] = InFactory;
	}
	else
	{
		Factories.Emplace(InFactoryName, InFactory);
	}

	Factories.ValueStableSort(
		[](const TSharedRef<IMediaImageViewerFactory>& InA, const TSharedRef<IMediaImageViewerFactory>& InB)
		{
			return InA->Priority < InB->Priority;
		}
	);
}

void FMediaViewerModule::UnregisterFactory(FName InFactoryName)
{
	Factories.Remove(InFactoryName);

	Factories.ValueStableSort(
		[](const TSharedRef<IMediaImageViewerFactory>& InA, const TSharedRef<IMediaImageViewerFactory>& InB)
		{
			return InA->Priority < InB->Priority;
		}
	);
}

bool FMediaViewerModule::OpenTab()
{
	return OpenTab({});
}

bool FMediaViewerModule::OpenTab(const FMediaViewerArgs& InMediaViewerArgs)
{
	TSharedPtr<SMediaViewerTab> ViewerTab = GetLiveTab();

	if (!ViewerTab.IsValid())
	{
		// This will potentially open the tab with the wrong settings.
		// It will be corrected later.
		ViewerTab = SpawnTab();

		if (!ViewerTab.IsValid())
		{
			return false;
		}
	}

	if (ViewerTab->GetArgs() == InMediaViewerArgs)
	{
		return true;
	}

	TSharedPtr<FMediaImageViewer> ImageViewers[static_cast<int32>(EMediaImageViewerPosition::COUNT)] =
	{
		ViewerTab->GetImageViewer(EMediaImageViewerPosition::First),
		ViewerTab->GetImageViewer(EMediaImageViewerPosition::Second)
	};

	ViewerTab = CreateMediaViewer(InMediaViewerArgs);
	ViewerTab->SetImageViewer(EMediaImageViewerPosition::First, ImageViewers[static_cast<int32>(EMediaImageViewerPosition::First)]);
	ViewerTab->SetImageViewer(EMediaImageViewerPosition::Second, ImageViewers[static_cast<int32>(EMediaImageViewerPosition::Second)]);

	SetTabBody(ViewerTab.ToSharedRef());

	return true;
}

TSharedRef<IMediaViewerLibrary> FMediaViewerModule::GetLibrary() const
{
	return Library;
}

TSharedPtr<FMediaViewerLibraryItem> FMediaViewerModule::CreateLibraryItem(const FAssetData& InAssetData) const
{
	for (const TPair<FName, TSharedRef<IMediaImageViewerFactory>>& FactoryPair : Factories)
	{
		if (FactoryPair.Value->SupportsAsset(InAssetData))
		{
			return FactoryPair.Value->CreateLibraryItem(InAssetData);
		}
	}

	return nullptr;
}

TSharedPtr<FMediaViewerLibraryItem> FMediaViewerModule::CreateLibraryItem(TNotNull<UObject*> InObject) const
{
	for (const TPair<FName, TSharedRef<IMediaImageViewerFactory>>& FactoryPair : Factories)
	{
		if (FactoryPair.Value->SupportsObject(InObject))
		{
			return FactoryPair.Value->CreateLibraryItem(InObject);
		}
	}

	return nullptr;
}

TSharedPtr<FMediaViewerLibraryItem> FMediaViewerModule::CreateLibraryItem(FName InItemType, const FMediaViewerLibraryItem& InSavedItem) const
{
	for (const TPair<FName, TSharedRef<IMediaImageViewerFactory>>& FactoryPair : Factories)
	{
		if (FactoryPair.Value->SupportsItemType(InItemType))
		{
			return FactoryPair.Value->CreateLibraryItem(InSavedItem);
		}
	}

	return nullptr;
}

TSharedRef<SMediaViewerTab> FMediaViewerModule::CreateMediaViewer(const FMediaViewerArgs& InArgs)
{
	return SNew(SMediaViewerTab, InArgs);
}

TSharedRef<IMediaViewerLibraryWidget> FMediaViewerModule::CreateLibrary(const IMediaViewerLibraryWidget::FArgs& InArgs)
{
	return SNew(SMediaViewerLibrary, InArgs, MakeShared<FMediaViewerDelegates>());
}

TSharedPtr<SDockTab> FMediaViewerModule::OpenStandaloneTab()
{
	TSharedPtr<FTabManager> TabManager = UE::MediaViewer::Private::GetLevelEditorTabManager();

	if (!TabManager.IsValid())
	{
		return nullptr;
	}

	TSharedRef<SMediaViewerTab> ViewerTab = CreateMediaViewer({});
	
	TSharedRef<SDockTab> StandaloneTab = SNew(SDockTab)
		.Label(LOCTEXT("MediaViewerTitleStandalone", "Media Viewer (Standalone)"))
		.ContentPadding(3.f)
		[
			// Create with default settings
			ViewerTab
		];

	StandaloneTab->SetTabIcon(FAppStyle::Get().GetBrush("Sequencer.Tracks.Media"));

	TabManager->InsertNewDocumentTab(
		TabId,
		StandaloneTabId,
		FTabManager::FLiveTabSearch(),
		StandaloneTab
	);

	return StandaloneTab;
}

bool FMediaViewerModule::HasImage(EMediaImageViewerPosition InPosition) const
{
	TSharedPtr<FMediaImageViewer> Viewer = GetImage(InPosition);

	return Viewer.IsValid() && Viewer != FNullImageViewer::GetNullImageViewer();
}

TSharedPtr<FMediaImageViewer> FMediaViewerModule::GetImage(EMediaImageViewerPosition InPosition) const
{
	if (TSharedPtr<SMediaViewerTab> ViewerTab = GetLiveTab())
	{
		return ViewerTab->GetImageViewer(InPosition);
	}

	return nullptr;
}

bool FMediaViewerModule::SetImage(EMediaImageViewerPosition InPosition, const FAssetData& InAssetData)
{
	TSharedPtr<SMediaViewerTab> ViewerTab = GetLiveTab();

	if (!ViewerTab.IsValid())
	{
		return false;
	}

	TSharedPtr<FMediaImageViewer> ImageViewer;

	for (const TPair<FName, TSharedRef<IMediaImageViewerFactory>>& FactoryPair : Factories)
	{
		if (FactoryPair.Value->SupportsAsset(InAssetData))
		{
			if (TSharedPtr<FMediaImageViewer> CreatedImageViewer = FactoryPair.Value->CreateImageViewer(InAssetData))
			{
				ImageViewer = CreatedImageViewer;
				break;
			}
		}
	}

	if (!ImageViewer)
	{
		return false;
	}

	ViewerTab->SetImageViewer(InPosition, ImageViewer.ToSharedRef());

	return true;
}

bool FMediaViewerModule::SetImage(EMediaImageViewerPosition InPosition, UObject* InObject)
{
	if (!InObject)
	{
		return false;
	}

	TNotNull<UObject*> NotNullObject = InObject;

	TSharedPtr<SMediaViewerTab> ViewerTab = GetLiveTab();

	if (!ViewerTab.IsValid())
	{
		return false;
	}

	TSharedPtr<FMediaImageViewer> ImageViewer;

	for (const TPair<FName, TSharedRef<IMediaImageViewerFactory>>& FactoryPair : Factories)
	{
		if (FactoryPair.Value->SupportsObject(NotNullObject))
		{
			if (TSharedPtr<FMediaImageViewer> CreatedImageViewer = FactoryPair.Value->CreateImageViewer(NotNullObject))
			{
				ImageViewer = CreatedImageViewer;
				break;
			}
		}
	}

	if (!ImageViewer)
	{
		return false;
	}

	ViewerTab->SetImageViewer(InPosition, ImageViewer.ToSharedRef());

	return true;
}

bool FMediaViewerModule::SetImage(EMediaImageViewerPosition InPosition, const TSharedRef<FMediaImageViewer> InImageViewer)
{
	if (TSharedPtr<SMediaViewerTab> ViewerTab = GetLiveTab())
	{
		ViewerTab->SetImageViewer(InPosition, InImageViewer);
		return true;
	}

	return false;
}

bool FMediaViewerModule::ClearImage(EMediaImageViewerPosition InPosition)
{
	if (TSharedPtr<SMediaViewerTab> ViewerTab = GetLiveTab())
	{
		ViewerTab->SetImageViewer(InPosition, nullptr);
		return true;
	}

	return false;
}

void FMediaViewerModule::StartupModule()
{
	FMediaViewerStyle::Get();
	FMediaViewerCommands::Register();
	RegisterDefaultImageViewers();
	FMediaViewerContentBrowserIntegration::Get()->Integrate();

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditorModule.OnLevelEditorCreated().AddLambda(
		[this](TSharedPtr<ILevelEditor> InLevelEditor)
		{
			if (InLevelEditor.IsValid())
			{
				RegisterLevelEditorMenuItems();
			}
		}
	);
}

void FMediaViewerModule::ShutdownModule()
{
	UnregisterAllImageViewers();
	UnregisterLevelEditorMenuItems();
	FMediaViewerCommands::Unregister();
	FMediaViewerContentBrowserIntegration::Get()->Disintegrate();
	FMediaViewerUtils::ReleasePreloadedMaterialAssets();
}

void FMediaViewerModule::OnTabClosed(TSharedRef<SDockTab> InDockTab)
{
	TSharedRef<SWidget> Content = InDockTab->GetContent();

	if (Content->GetWidgetClass().GetWidgetType() == SMediaViewerTab::StaticWidgetClass().GetWidgetType())
	{
		TSharedRef<SMediaViewerTab> MediaViewerTab = StaticCastSharedRef<SMediaViewerTab>(Content);
		MediaViewerTab->GetViewer()->SaveLastOpenedState(/* Save config */ false);

		UMediaViewerLibraryIni& Ini = UMediaViewerLibraryIni::Get();
		Ini.SaveLibrary(MediaViewerTab->GetLibrary());
		Ini.SaveConfig();
	}
}

void FMediaViewerModule::RegisterLevelEditorMenuItems()
{
	if (TSharedPtr<FTabManager> TabManager = UE::MediaViewer::Private::GetLevelEditorTabManager())
	{
		UnregisterLevelEditorMenuItems();

		FTabSpawnerEntry& TabSpawnerEntry = TabManager->RegisterTabSpawner(
			TabId,
			FOnSpawnTab::CreateRaw(this, &FMediaViewerModule::CreateTab))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.Tracks.Media"))
			.SetDisplayName(LOCTEXT("OpenMediaViewer", "Media Viewer"))
			.SetTooltipText(LOCTEXT("OpenMediaViewerTooltip", "Open the Media Viewer"))
			.SetMenuType(ETabSpawnerMenuType::Enabled)
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory());
	}
}

void FMediaViewerModule::UnregisterLevelEditorMenuItems()
{
	if (TSharedPtr<FTabManager> TabManager = UE::MediaViewer::Private::GetLevelEditorTabManager())
	{
		TabManager->UnregisterTabSpawner(TabId);
	}
}

TSharedPtr<SMediaViewerTab> FMediaViewerModule::GetLiveTab() const
{
	TSharedPtr<FTabManager> TabManager = UE::MediaViewer::Private::GetLevelEditorTabManager();

	if (!TabManager.IsValid())
	{
		return nullptr;
	}

	const FName ResolvedTabId = TabId;

	if (TSharedPtr<SDockTab> Tab = TabManager->FindExistingLiveTab(ResolvedTabId))
	{
		TSharedRef<SWidget> TabContent = Tab->GetContent();

		if (TabContent->GetWidgetClass().GetWidgetType() == SMediaViewerTab::StaticWidgetClass().GetWidgetType())
		{
			return StaticCastSharedRef<SMediaViewerTab>(TabContent);
		}
	}

	return nullptr;
}

TSharedPtr<SMediaViewerTab> FMediaViewerModule::SpawnTab() const
{
	TSharedPtr<FTabManager> TabManager = UE::MediaViewer::Private::GetLevelEditorTabManager();

	if (!TabManager.IsValid())
	{
		return nullptr;
	}

	const FName ResolvedTabId = TabId;

	if (TSharedPtr<SDockTab> Tab = TabManager->TryInvokeTab(ResolvedTabId))
	{
		TSharedRef<SWidget> TabContent = Tab->GetContent();

		if (TabContent->GetWidgetClass().GetWidgetType() == SMediaViewerTab::StaticWidgetClass().GetWidgetType())
		{
			return StaticCastSharedRef<SMediaViewerTab>(TabContent);
		}
	}

	return nullptr;
}

TSharedRef<SDockTab> FMediaViewerModule::CreateTab(const FSpawnTabArgs& InArgs)
{
	TSharedRef<SMediaViewerTab> ViewerTab = CreateMediaViewer({});

	return SNew(SDockTab)
		.Label(LOCTEXT("MediaViewerTitle", "Media Viewer"))
		.ContentPadding(3.f)
		.OnTabClosed_Static(&FMediaViewerModule::OnTabClosed)
		[
			// Create with default settings
			ViewerTab
		];
}

bool FMediaViewerModule::SetTabBody(const TSharedRef<SMediaViewerTab>& InViewerTab)
{
	TSharedPtr<FTabManager> TabManager = UE::MediaViewer::Private::GetLevelEditorTabManager();

	if (!TabManager.IsValid())
	{
		return false;
	}

	const FName ResolvedTabId = TabId;

	if (TSharedPtr<SDockTab> Tab = TabManager->FindExistingLiveTab(ResolvedTabId))
	{
		Tab->SetContent(InViewerTab);
	}

	return false;
}

void FMediaViewerModule::RegisterDefaultImageViewers()
{
	RegisterFactory(FColorImageViewer::ItemTypeName,                 MakeShared<FColorImageViewer::FFactory>());
	RegisterFactory(FMaterialInterfaceImageViewer::ItemTypeName,     MakeShared<FMaterialInterfaceImageViewer::FFactory>());
	RegisterFactory(FMediaSourceImageViewer::ItemTypeName_Asset,     MakeShared<FMediaSourceImageViewer::FFactory>());
	RegisterFactory(FMediaTextureImageViewer::ItemTypeName,          MakeShared<FMediaTextureImageViewer::FFactory>());
	RegisterFactory(FTexture2DImageViewer::ItemTypeName,             MakeShared<FTexture2DImageViewer::FFactory>());
	RegisterFactory(FTextureRenderTarget2DImageViewer::ItemTypeName, MakeShared<FTextureRenderTarget2DImageViewer::FFactory>());
}

void FMediaViewerModule::UnregisterAllImageViewers()
{
	Factories.Reset();
}

void FMediaViewerModule::SaveHistory(const TSharedRef<SMediaViewerLibrary> InLibrary)
{
}

void FMediaViewerModule::LoadHistory(const TSharedRef<SMediaViewerLibrary> InLibrary)
{
}

} // UE::MediaViewer::Private

IMPLEMENT_MODULE(UE::MediaViewer::Private::FMediaViewerModule, MediaViewer)

#undef LOCTEXT_NAMESPACE
