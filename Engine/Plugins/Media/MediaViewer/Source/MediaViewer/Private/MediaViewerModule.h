// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaViewerModule.h"

#include "Containers/Map.h"

class FSpawnTabArgs;
class SDockTab;

namespace UE::MediaViewer
{
	class IMediaViewerLibrary;
	class SMediaViewerTab;
	struct FMediaViewerArgs;
}

namespace UE::MediaViewer::Private
{

class SMediaViewerLibrary;

class FMediaViewerModule : public IMediaViewerModule
{
public:
	static const FLazyName TabId;
	static const FLazyName StandaloneTabId;

	FMediaViewerModule();

	//~ Begin IMediaViewerModule
	virtual bool IsFactoryRegistered(FName InFactoryName) const override;
	virtual bool HasFactoryFor(const FAssetData& InAssetData) const override;
	virtual bool HasFactoryFor(UObject* InObject) const override;
	virtual void RegisterFactory(FName InFactoryName, const TSharedRef<IMediaImageViewerFactory>& InFactory) override;
	virtual void UnregisterFactory(FName InFactoryName) override;
	virtual bool OpenTab() override;
	virtual bool OpenTab(const FMediaViewerArgs& InMediaViewerArgs) override;
	virtual TSharedRef<IMediaViewerLibrary> GetLibrary() const;
	virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(const FAssetData& InAssetData) const override;
	virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(TNotNull<UObject*> InObject) const override;
	virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(FName InItemType, const FMediaViewerLibraryItem& InSavedItem) const override;
	virtual bool HasImage(EMediaImageViewerPosition InPosition) const override;
	virtual TSharedPtr<FMediaImageViewer> GetImage(EMediaImageViewerPosition InPosition) const override;
	virtual bool SetImage(EMediaImageViewerPosition InPosition, const FAssetData& InAssetData) override;
	virtual bool SetImage(EMediaImageViewerPosition InPosition, UObject* InObject) override;
	virtual bool SetImage(EMediaImageViewerPosition InPosition, const TSharedRef<FMediaImageViewer> InImageViewer) override;
	virtual bool ClearImage(EMediaImageViewerPosition InPosition) override;
	virtual TSharedRef<SMediaViewerTab> CreateMediaViewer(const FMediaViewerArgs& InMediaViewerArgs) override;
	virtual TSharedRef<IMediaViewerLibraryWidget> CreateLibrary(const IMediaViewerLibraryWidget::FArgs& InArgs) override;
	virtual TSharedPtr<SDockTab> OpenStandaloneTab();
	//~ End IMediaViewerModule

	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

private:
	static void OnTabClosed(TSharedRef<SDockTab> InDockTab);

	TMap<FName, TSharedRef<IMediaImageViewerFactory>> Factories;
	TSharedRef<IMediaViewerLibrary> Library;

	void RegisterLevelEditorMenuItems();
	void UnregisterLevelEditorMenuItems();

	TSharedPtr<SMediaViewerTab> GetLiveTab() const;

	TSharedPtr<SMediaViewerTab> SpawnTab() const;

	TSharedRef<SDockTab> CreateTab(const FSpawnTabArgs& InArgs);

	bool SetTabBody(const TSharedRef<SMediaViewerTab>& InViewerTab);

	void RegisterDefaultImageViewers();
	void UnregisterAllImageViewers();

	void SaveHistory(const TSharedRef<SMediaViewerLibrary> InLibrary);

	void LoadHistory(const TSharedRef<SMediaViewerLibrary> InLibrary);
};

} // UE::MediaViewer::Private
