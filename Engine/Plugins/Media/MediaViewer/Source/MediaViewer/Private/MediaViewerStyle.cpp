// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaViewerStyle.h"

#include "DetailLayoutBuilder.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Styling/ToolBarStyle.h"

namespace UE::MediaViewer::Private
{

const FName FMediaViewerStyle::StyleName("MediaViewerStyle");

FMediaViewerStyle::FMediaViewerStyle()
	: FSlateStyleSet(StyleName)
{
	const FVector2D Image16x16(16);
	const FVector2D Image20x20(20);

	FSlateStyleSet::SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	FSlateStyleSet::SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	FTextBlockStyle NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");
	NormalText.SetFont(IDetailLayoutBuilder::GetDetailFont());
	NormalText.SetShadowColorAndOpacity(FStyleColors::Panel.GetSpecifiedColor());
	NormalText.SetShadowOffset(FVector2D(1.0, 1.0));
	Set("RichTextBlock.Normal", NormalText);
	Set("RichTextBlock.Red", FTextBlockStyle(NormalText).SetColorAndOpacity(FLinearColor(1.0f, 0.1f, 0.1f)));
	Set("RichTextBlock.Green", FTextBlockStyle(NormalText).SetColorAndOpacity(FLinearColor(0.1f, 1.0f, 0.1f)));
	Set("RichTextBlock.Blue", FTextBlockStyle(NormalText).SetColorAndOpacity(FLinearColor(0.1f, 0.1f, 1.0f)));

	FSlateBrush* BrushTableRowOdd = new FSlateBrush();
	BrushTableRowOdd->TintColor = FLinearColor(0.5f, 0.5f, 0.5f, 1.0f);
	Set("TableRowOdd", BrushTableRowOdd);

	FButtonStyle LibraryButtonStyle = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button");
	LibraryButtonStyle.SetNormalPadding(FMargin(3.f, 0.f, 3.f, 0.f));
	LibraryButtonStyle.SetPressedPadding(FMargin(3.f, 0.f, 3.f, 0.f));
	Set("LibraryButtonStyle", LibraryButtonStyle);

	FButtonStyle ToolbarButtonStyle = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button");
	ToolbarButtonStyle.SetNormalPadding(FMargin(0.f, 0.f, 0.f, 0.f));
	ToolbarButtonStyle.SetPressedPadding(FMargin(0.f, 0.f, 0.f, 0.f));
	Set("ToolbarButtonStyle", ToolbarButtonStyle);

	Set("MediaButtons", FButtonStyle(FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Animation.PlayControlsButton"))
		.SetNormal(FSlateNoResource())
		.SetDisabled(FSlateNoResource())
		.SetHovered(FSlateRoundedBoxBrush(FLinearColor(0.2f, 0.2f, 0.2f, 0.5f), 3.f, Image20x20))
		.SetPressed(FSlateRoundedBoxBrush(FLinearColor(0.1f, 0.1f, 0.1f, 0.5f), 3.f, Image20x20))
		.SetNormalPadding(FMargin(2.f, 2.f, 2.f, 2.f))
		.SetPressedPadding(FMargin(2.f, 2.f, 2.f, 2.f)));

	Set("LibraryButtons", FButtonStyle(FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Animation.PlayControlsButton"))
		.SetNormal(FSlateNoResource())
		.SetDisabled(FSlateNoResource())
		.SetHovered(FSlateRoundedBoxBrush(FLinearColor(0.2f, 0.2f, 0.2f, 0.5f), 3.f, Image20x20))
		.SetPressed(FSlateRoundedBoxBrush(FLinearColor(0.1f, 0.1f, 0.1f, 0.5f), 3.f, Image20x20))
		.SetNormalPadding(FMargin(2.f, 0.f, 2.f, 0.f))
		.SetPressedPadding(FMargin(2.f, 0.f, 2.f, 0.f)));

	if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME))
	{
		SetContentRoot(FPaths::Combine(Plugin->GetBaseDir(), TEXT("Content")));
		Set("ColorChip", new IMAGE_BRUSH_SVG(TEXT("color-chip"), FVector2D(12.0)));
	}

	// Commands
	FSlateStyleSet::SetContentRoot(FPaths::EngineContentDir() / "Slate");

	Set("MediaViewer.CopyTransform", new IMAGE_BRUSH_SVG("Starship/Common/Copy",        Image16x16));
	Set("MediaViewer.ToggleOverlay", new IMAGE_BRUSH_SVG("Starship/Common/visible",     Image16x16));

	FSlateStyleSet::SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));

	Set("MediaViewer.ResetTransform",        new IMAGE_BRUSH_SVG("Starship/Common/ResetCamera",      Image16x16));
	Set("MediaViewer.AddToLibrary",          new IMAGE_BRUSH_SVG("Starship/Common/Unpinned",         Image16x16));
	Set("MediaViewer.ToggleLockedTransform", new IMAGE_BRUSH(    "Common/link",                      Image16x16));
	Set("MediaViewer.ResetAllTransforms",    new IMAGE_BRUSH_SVG("Starship/Common/ResetCamera",      Image16x16));
	Set("MediaViewer.SwapAB",                new IMAGE_BRUSH(    "Icons/Paint/Paint_SwapColors_40x", Image16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FMediaViewerStyle::~FMediaViewerStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FMediaViewerStyle& FMediaViewerStyle::Get()
{
	static FMediaViewerStyle Instance;
	return Instance;
}

} // UE::MediaViewer::Private
