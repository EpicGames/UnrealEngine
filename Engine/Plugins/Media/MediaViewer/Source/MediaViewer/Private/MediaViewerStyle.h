// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

namespace UE::MediaViewer::Private
{

/**
 * Style declarations for the image widgets.
 */
class FMediaViewerStyle : public FSlateStyleSet
{
public:
	static const FName StyleName;

	static FMediaViewerStyle& Get();

private:
	FMediaViewerStyle();
	virtual ~FMediaViewerStyle() override;
};

} // UE::MediaViewer::Private
