// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaViewerUtils.h"

#include "CanvasTypes.h"
#include "ColorManagement/TransferFunctions.h"
#include "DetailsViewArgs.h"
#include "EditorClassUtils.h"
#include "Engine/Canvas.h"
#include "IStructureDetailsView.h"
#include "MaterialShared.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "Math/IntPoint.h"
#include "MediaViewer.h"
#include "Misc/TVariant.h"
#include "Modules/ModuleManager.h"
#include "PixelFormat.h"
#include "PropertyEditorModule.h"
#include "RHIShaderPlatform.h"
#include "ShaderCompiler.h"
#include "Slate/WidgetRenderer.h"
#include "SlateMaterialBrush.h"
#include "TextureResource.h"
#include "ThumbnailRendering/ThumbnailRenderer.h"
#include "UObject/Package.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/Images/SImage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaViewerUtils)

namespace UE::MediaViewer::Private
{

namespace PreloadedMaterials
{
	TStrongObjectPtr<UMaterial> MipRenderMaterial;

	/**
	 * Utility function to conditionally load the top level asset at the given path and start compiling it
	 * for the given platform.
	 * @param InMaterialPath Path to a top level material asset.
	 * @param InOutMaterial Storage for the loaded material, function will no-op if material is already loaded. 
	 */
	void ConditionalPreloadAndCompileMaterial(const TCHAR* InMaterialPath, TStrongObjectPtr<UMaterial>& InOutMaterial)
	{
		// Make sure we have the correct path to avoid stale materials.
		if (InOutMaterial.IsValid() && FSoftObjectPath(InOutMaterial.Get()) != FSoftObjectPath(InMaterialPath))
		{
			UE_LOGF(LogMediaViewer, Error, "Preloaded material path mismatch: expected \"%ls\", got \"%ls\". Discarding for reload.",
				InMaterialPath, *InOutMaterial->GetPathName());
			InOutMaterial.Reset();
		}

		if (!InOutMaterial.IsValid())
		{
			InOutMaterial.Reset(LoadObject<UMaterial>(nullptr, InMaterialPath));

			if (!InOutMaterial.IsValid())
			{
				UE_LOGF(LogMediaViewer, Error, "Failed to preload material \"%ls\"", InMaterialPath);
				return;
			}

			FMediaViewerUtils::ConditionallyCompileMaterial(InOutMaterial.Get());
		}
	}
}

void FMediaViewerUtils::PreloadMaterialAssets()
{
	PreloadedMaterials::ConditionalPreloadAndCompileMaterial(MipRenderMaterialPath, PreloadedMaterials::MipRenderMaterial);
	// Place media viewer default materials that need preloading here and corresponding reset in ReleasePreloadedMaterialAssets().
}

void FMediaViewerUtils::ReleasePreloadedMaterialAssets()
{
	PreloadedMaterials::MipRenderMaterial.Reset();
}

void FMediaViewerUtils::ConditionallyCompileMaterial(TNotNull<UMaterialInterface*> InMaterial)
{
	// Non-blocking shader-compile kick for the current rendering platform.
	if (UMaterial* BaseMaterial = InMaterial->GetMaterial())
	{
		if (FMaterialResource* Resource = BaseMaterial->GetMaterialResource(GMaxRHIShaderPlatform))
		{
			if (!Resource->IsGameThreadShaderMapComplete())
			{
				Resource->SubmitCompileJobs_GameThread(EShaderCompileJobPriority::Normal);
			}
		}
	}
}

bool FMediaViewerUtils::IsMaterialShaderMapComplete(TNotNull<UMaterialInterface*> InMaterial)
{
	if (const UMaterial* BaseMaterial = InMaterial->GetMaterial())
	{
		if (const FMaterialResource* Resource = BaseMaterial->GetMaterialResource(GMaxRHIShaderPlatform))
		{
			return Resource->IsGameThreadShaderMapComplete();
		}
	}

	return false;
}

TOptional<TVariant<FColor, FLinearColor>> FMediaViewerUtils::GetPixelColor(TArrayView64<uint8> InPixelData,
	EPixelFormat InPixelFormat, const FIntPoint& InTextureSize, const FIntPoint& InPixelCoords, int32 InMipLevel)
{
	// Ensure at least 1 byte per pixel...
	if (InPixelData.Num() < (InTextureSize.X * InTextureSize.Y))
	{
		return {};
	}

	TVariant<FColor, FLinearColor> ColorVariant;

	const int32 MipSize = InPixelData.Num();
	const int32 StripeSize = MipSize / InTextureSize.Y;
	const int32 StripeOffset = StripeSize * InPixelCoords.Y;

	switch (InPixelFormat)
	{
		case EPixelFormat::PF_R8:
		{
			const int32 PixelOffset = StripeOffset + InPixelCoords.X;
			FColor Color;
			Color.R = InPixelData[PixelOffset];
			Color.G = 0;
			Color.B = 0;
			Color.A = 255;
			ColorVariant.Set<FColor>(Color);
			break;
		}

		case EPixelFormat::PF_G8:
		{
			const int32 PixelOffset = StripeOffset + InPixelCoords.X;
			FColor Color;
			Color.R = 0;
			Color.G = InPixelData[PixelOffset];
			Color.B = 0;
			Color.A = 255;
			ColorVariant.Set<FColor>(Color);
			break;
		}

		case EPixelFormat::PF_A8:
		{
			const int32 PixelOffset = StripeOffset + InPixelCoords.X;
			FColor Color;
			Color.R = 0;
			Color.G = 0;
			Color.B = 0;
			Color.A = InPixelData[PixelOffset];
			ColorVariant.Set<FColor>(Color);
			break;
		}

		case EPixelFormat::PF_R8G8:
		{
			const int32 PixelOffset = StripeOffset + (InPixelCoords.X * 2);
			FColor Color;
			Color.R = InPixelData[PixelOffset];
			Color.G = InPixelData[PixelOffset + 1];
			Color.B = 0;
			Color.A = 255;
			ColorVariant.Set<FColor>(Color);
			break;
		}

		case EPixelFormat::PF_R8G8B8:
		{
			const int32 PixelOffset = StripeOffset + (InPixelCoords.X * 3);
			FColor Color;
			Color.R = InPixelData[PixelOffset];
			Color.G = InPixelData[PixelOffset + 1];
			Color.B = InPixelData[PixelOffset + 2];
			Color.A = 255;
			ColorVariant.Set<FColor>(Color);
			break;
		}

		case EPixelFormat::PF_R8G8B8A8:
		{
			const int32 PixelOffset = StripeOffset + (InPixelCoords.X * 4);
			FColor Color;
			Color.R = InPixelData[PixelOffset];
			Color.G = InPixelData[PixelOffset + 1];
			Color.B = InPixelData[PixelOffset + 2];
			Color.A = InPixelData[PixelOffset + 3];
			ColorVariant.Set<FColor>(Color);
			break;
		}

		case EPixelFormat::PF_B8G8R8A8:
		{
			const int32 PixelOffset = StripeOffset + (InPixelCoords.X * 4);
			FColor Color;
			Color.R = InPixelData[PixelOffset + 2];
			Color.G = InPixelData[PixelOffset + 1];
			Color.B = InPixelData[PixelOffset];
			Color.A = InPixelData[PixelOffset + 3];
			ColorVariant.Set<FColor>(Color);
			break;
		}

		case EPixelFormat::PF_A8R8G8B8:
		{
			const int32 PixelOffset = StripeOffset + (InPixelCoords.X * 4);
			FColor Color;
			Color.R = InPixelData[PixelOffset + 1];
			Color.G = InPixelData[PixelOffset + 2];
			Color.B = InPixelData[PixelOffset + 3];
			Color.A = InPixelData[PixelOffset];
			ColorVariant.Set<FColor>(Color);
			break;
		}

		default:
			return {};
	}

	return ColorVariant;
}

UTextureRenderTarget2D* FMediaViewerUtils::CreateRenderTarget(const FIntPoint& InSize, bool bInTransparent, ETextureRenderTargetFormat InFormat)
{
	UTextureRenderTarget2D* RenderTarget = NewObject<UTextureRenderTarget2D>(GetTransientPackage());
	check(RenderTarget);
	RenderTarget->RenderTargetFormat = InFormat;
	RenderTarget->bAutoGenerateMips = false;
	RenderTarget->bCanCreateUAV = false;
	RenderTarget->ClearColor = bInTransparent ? FLinearColor::Transparent : FLinearColor::Black;
	RenderTarget->InitAutoFormat(InSize.X, InSize.Y);
	RenderTarget->UpdateResourceImmediate(true);
	RenderTarget->AddAssetUserData(NewObject<UMediaViewerUserData>(RenderTarget));

	return RenderTarget;
}

UTextureRenderTarget2D* FMediaViewerUtils::RenderMaterial(TNotNull<UMaterialInterface*> InMaterial)
{
	UTextureRenderTarget2D* RenderTarget = CreateRenderTarget(FIntPoint(256, 256), InMaterial->IsUIMaterial());

	if (!RenderMaterial(InMaterial, TNotNull<UTextureRenderTarget2D*>(RenderTarget)))
	{
		UE_LOGF(LogMediaViewer, Error, "Failed to render material %ls.", *InMaterial->GetName());
	}

	return RenderTarget;
}

bool FMediaViewerUtils::RenderMaterial(
	const TNotNull<UMaterialInterface*> InMaterial,
	const TNotNull<UTextureRenderTarget2D*> InRenderTarget,
	const EShaderCompileFlags InFlags)
{
	if (!InMaterial->IsUIMaterial())
	{
		return false;
	}
	
	if (EnumHasAnyFlags(InFlags, EShaderCompileFlags::EnsureIsComplete))
	{
		InMaterial->EnsureIsComplete();
	}
	else if (!IsMaterialShaderMapComplete(InMaterial))
	{
		return false;
	}

	// Taken from UMaterialInstanceThumbnailRenderer - but with the background checkerboard texture removed.
	constexpr bool bUseGammaCorrection = true;
	FWidgetRenderer WidgetRenderer(bUseGammaCorrection);

	const FVector2D DrawSize = FVector2D(InRenderTarget->GetSurfaceWidth(), InRenderTarget->GetSurfaceHeight());

	FSlateMaterialBrush UIMaterialBrush(DrawSize);
	UIMaterialBrush.SetMaterial(InMaterial);

	TSharedRef<SImage> Image = SNew(SImage)
		.Image(&UIMaterialBrush);

	constexpr float DeltaTime = 0.f;
	WidgetRenderer.DrawWidget(InRenderTarget, Image, DrawSize, DeltaTime);

	return true;
}

TSharedRef<IStructureDetailsView> FMediaViewerUtils::CreateStructDetailsView(TSharedRef<FStructOnScope> InStructOnScope,
	const FText& InCustomName, FNotifyHook* InNotifyHook)
{
	FDetailsViewArgs DetailsArgs;
	DetailsArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsArgs.bAllowFavoriteSystem = false;
	DetailsArgs.bAllowMultipleTopLevelObjects = false;
	DetailsArgs.bAllowSearch = false;
	DetailsArgs.bCustomFilterAreaLocation = false;
	DetailsArgs.bCustomNameAreaLocation = false;
	DetailsArgs.bShowOptions = false;
	DetailsArgs.bShowObjectLabel = false;
	DetailsArgs.bShowPropertyMatrixButton = false;
	DetailsArgs.bShowScrollBar = false;
	DetailsArgs.bShowSectionSelector = false;
	DetailsArgs.bUpdatesFromSelection = false;
	DetailsArgs.NotifyHook = InNotifyHook;

	FStructureDetailsViewArgs StructArgs;
	StructArgs.bShowAssets = true;
	StructArgs.bShowClasses = true;
	StructArgs.bShowInterfaces = true;
	StructArgs.bShowObjects = true;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	return PropertyEditorModule.CreateStructureDetailView(DetailsArgs, StructArgs, InStructOnScope, InCustomName);
}

FColor FMediaViewerUtils::CorrectGamma(const FColor& InColor)
{
	return CorrectGamma(InColor.ReinterpretAsLinear()).ToFColor(/* SRGB */ false);
}

FLinearColor FMediaViewerUtils::CorrectGamma(const FLinearColor& InColor)
{
	FLinearColor Color;
	Color.R = UE::Color::EncodeSRGB(InColor.R);
	Color.G = UE::Color::EncodeSRGB(InColor.G);
	Color.B = UE::Color::EncodeSRGB(InColor.B);
	Color.A = InColor.A;

	return Color;
}

} // UE::MediaViewer::Private
