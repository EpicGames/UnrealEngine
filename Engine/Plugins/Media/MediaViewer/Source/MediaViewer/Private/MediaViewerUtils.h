// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetUserData.h"

#include "Containers/ArrayView.h"
#include "DetailsViewArgs.h"
#include "Engine/TextureRenderTarget2D.h"
#include "HAL/Platform.h"
#include "IStructureDetailsView.h"
#include "Math/Color.h"
#include "Math/MathFwd.h"
#include "Misc/Optional.h"
#include "PropertyEditorModule.h"
#include "Templates/SharedPointerFwd.h"
#include "Templates/Tuple.h"

#include "MediaViewerUtils.generated.h"

class FNotifyHook;
class FStructOnScope;
class FText;
class IStructureDetailsView;
class UMaterialInterface;
class UTextureRenderTarget2D;
enum EPixelFormat : uint8;
struct FDetailsViewArgs;
struct FStructureDetailsViewArgs;
template <typename T, typename... Ts> class TVariant;

UCLASS()
class UMediaViewerUserData : public UAssetUserData
{
	GENERATED_BODY()
};

namespace UE::MediaViewer::Private
{
/**
 * Render Material optional flags.
 */
enum class EShaderCompileFlags : uint8
{
	None = 0,
	/**
	 * Flag that controls how the shader-readiness gate is enforced:
	 * - if present (default): block on UMaterialInterface::EnsureIsComplete before rendering. The RT
	 *     is guaranteed populated on success. Use this for one-shot callers and any caller that
	 *     can't gracefully handle a "not yet" result.     
	 * - if not present: non-blocking. If the shader map for the current platform isn't complete yet
	 *     (@see IsMaterialShaderMapComplete) the call returns false with the RT untouched, and
	 *     the caller is expected to retry on a later frame. Use this for per-frame / mip-cache
	 *     paths where a synchronous compile stall is undesirable.
	 */
	EnsureIsComplete = 1<<0,

	Default = EnsureIsComplete
};
ENUM_CLASS_FLAGS(EShaderCompileFlags);

DECLARE_DELEGATE_OneParam(FRenderComplete, bool /* Was successfully rendered */)

class FMediaViewerUtils
{
public:
	/** Path to the UI material used by FMediaImageViewer to sample a texture at a specific mip level. */
	static constexpr const TCHAR* MipRenderMaterialPath = TEXT("/Script/Engine.Material'/MediaViewer/M_TextureMip.M_TextureMip'");

	/**
	 * Loads the materials this module needs to render with and kicks off their shader
	 * compilation in the background, so the shader maps are ready before any user-facing
	 * code path tries to use them. Strong references are kept for the rest of the editor
	 * session. Safe to call repeatedly; each load only happens once.
	 */
	static void PreloadMaterialAssets();

	/**
	 * Releases the strong references held by PreloadMaterialAssets. Must be called from
	 * the module's ShutdownModule (i.e. before UObject teardown) so we don't reach into the
	 * UObject system from a static destructor at process exit.
	 */
	static void ReleasePreloadedMaterialAssets();

	/**
	 * If the material's shader map for the current rendering platform is not complete yet,
	 * submits an async compile job at Normal priority. Does not block the game thread:
	 * callers needing the material to actually render should fall back gracefully until
	 * the shader map reports complete.
	 */
	static void ConditionallyCompileMaterial(TNotNull<UMaterialInterface*> InMaterial);

	/**
	 * Returns true only when the material's shader map for the current rendering platform
	 * has been confirmed complete. Returns false in every other case, including:
	 *   - the material has no parent UMaterial, or
	 *   - the parent has no FMaterialResource for the current platform yet (a genuine
	 *     not-ready signal that can appear, for example, after a feature-level switch).
	 *
	 * Use to gate draw calls that depend on the shaders being ready without forcing a
	 * synchronous compile. IsCompiling() / IsComplete() on a freshly created MID are
	 * unreliable (see the comment on UMaterialInterface::EnsureIsComplete) - this is why
	 * we query the parent's FMaterialResource directly.
	 */
	static bool IsMaterialShaderMapComplete(TNotNull<UMaterialInterface*> InMaterial);

	/** Retrieves the pixel color of a given pixel based on its pixel format. Or none if out of range. */
	static TOptional<TVariant<FColor, FLinearColor>> GetPixelColor(TArrayView64<uint8> InPixelData, EPixelFormat InPixelFormat, 
		const FIntPoint& InTextureSize, const FIntPoint& InPixelCoords, int32 InMipLevel);

	/** 
	 * Creates a render target texture parented to the transient package. 
	 * The target will have the @see UMediaViewerUserData asset data to differentiate it from other render targets.
	 */
	static UTextureRenderTarget2D* CreateRenderTarget(const FIntPoint& InSize, bool bInTransparent, ETextureRenderTargetFormat InFormat = ETextureRenderTargetFormat::RTF_RGBA8);

	/**
	 * Creates a 256x256 render target (@see CreateRenderTarget, transparent if the material is a
	 * UI material) and renders the material to it. Inherits the two-arg overload's default
	 * blocking behavior, so the returned RT is guaranteed populated on first use - intended for
	 * one-shot callers (thumbnails, brush creation) that have no retry path.
	 */
	static UTextureRenderTarget2D* RenderMaterial(TNotNull<UMaterialInterface*> InMaterial);

	/**
	 * Renders the given material to the given render target using FWidgetRenderer and an
	 * FSlateMaterialBrush. @see EShaderCompileFlags for blocking vs non-blocking behavior.
	 *
	 * @return true if the render was issued, false if it was skipped. False can mean either the
	 *   material is not a UI material, or (when EShaderCompileFlags::EnsureIsComplete is not set)
	 *   the shader map wasn't complete. In the latter case the caller MUST treat the return as
	 *   "try again later" - caching it as success will latch a transient not-ready result into
	 *   a permanent black RT.
	 */
	static bool RenderMaterial(
		const TNotNull<UMaterialInterface*> InMaterial,
		const TNotNull<UTextureRenderTarget2D*> InRenderTarget,
		const EShaderCompileFlags InFlags = EShaderCompileFlags::Default);

	/** Creates a struct details view based on the given struct. The view will have most settings disabled for a clean view. */
	static TSharedRef<IStructureDetailsView> CreateStructDetailsView(TSharedRef<FStructOnScope> InStructOnScope, const FText& InCustomName,
		FNotifyHook* InNotifyHook = nullptr);

	/**
	 * Corrects the gamma levels in the given color.
	 * 
	 * @return The corrected color.
	 */
	static FColor CorrectGamma(const FColor& InColor);

	/**
	 * Corrects the gamma levels in the given color.
	 *
	 * @return The corrected color.
	 */
	static FLinearColor CorrectGamma(const FLinearColor& InColor);
};
} // UE::MediaViewer::Private
