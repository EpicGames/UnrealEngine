// Copyright Epic Games, Inc. All Rights Reserved.

#include "Providers/ViewerTileVisibilityProvider.h"

#include "Assets/MediaSequenceInfo.h"
#include "Math/UnrealMathUtility.h"

namespace UE::MediaViewer::Private
{

namespace
{
	/**
	 * Compute the LOD-derived mip pair (floor / ceil) from the consumer-pushed inputs.
	 * Returns true on success. Returns false when the inputs are degenerate (zero-sized
	 * rect or display) - callers should treat that as "no displayed mip yet".
	 *
	 * Output mips are clamped to [0, INT32_MAX); the caller is responsible for clamping
	 * against the actual NumMipLevels of the target sequence/texture.
	 */
	bool ComputeMipPair(const FViewerTileVisibilityProvider::FUpdateInputs& InInputs, int32& OutMipFloor, int32& OutMipCeil)
	{
		if (InInputs.MipOverride.IsSet())
		{
			const int32 Mip = FMath::Max(0, InInputs.MipOverride.GetValue());
			OutMipFloor = OutMipCeil = Mip;
			return true;
		}

		const FVector2D RectSize = InInputs.VisibleSourceRect.GetSize();
		if (RectSize.X <= 0.0 || RectSize.Y <= 0.0
			|| InInputs.DisplayedSizePx.X <= 0.0 || InInputs.DisplayedSizePx.Y <= 0.0)
		{
			return false;
		}

		const double RatioX = RectSize.X / InInputs.DisplayedSizePx.X;
		const double RatioY = RectSize.Y / InInputs.DisplayedSizePx.Y;
		const double Ratio = FMath::Min(RatioX, RatioY);

		// Epsilon avoids issuing an extra mip-1 fetch at near-1:1 zoom when FP noise nudges
		// Ratio just past 1.0 (Log2 ~ 1e-7 would still split Floor=0 / Ceil=1).
		if (Ratio <= 1.0 + UE_DOUBLE_KINDA_SMALL_NUMBER)
		{
			OutMipFloor = OutMipCeil = 0;
			return true;
		}

		// Stay in double through floor/ceil; a borderline log2 like 3.0 - epsilon would otherwise drop a mip.
		const double LOD = FMath::Log2(Ratio);
		OutMipFloor = FMath::Max(0, FMath::FloorToInt32(LOD));
		OutMipCeil  = FMath::Max(0, FMath::CeilToInt32 (LOD));
		return true;
	}
}

void FViewerTileVisibilityProvider::Update(const FUpdateInputs& InInputs)
{
	bool bChanged = false;
	{
		FScopeLock Lock(&Mutex);
		const bool bSame =
			Inputs.VisibleSourceRect == InInputs.VisibleSourceRect
			&& Inputs.DisplayedSizePx == InInputs.DisplayedSizePx
			&& Inputs.MipOverride == InInputs.MipOverride;
		if (!bSame)
		{
			Inputs = InInputs;
			bChanged = true;
		}
	}
	// Broadcast outside the mutex.
	if (bChanged)
	{
		BroadcastOnInputsChanged();
	}
}

int32 FViewerTileVisibilityProvider::GetDisplayedMipFloor() const
{
	FUpdateInputs Snapshot;
	{
		FScopeLock Lock(&Mutex);
		Snapshot = Inputs;
	}

	int32 MipFloor = 0;
	int32 MipCeil = 0;
	if (!ComputeMipPair(Snapshot, MipFloor, MipCeil))
	{
		return INDEX_NONE;
	}
	return MipFloor;
}

void FViewerTileVisibilityProvider::GatherVisibleTiles(const FMediaSequenceInfo& InSequenceInfo,
	FMediaTileVisibilityRequest& OutRequest) const
{
	if (InSequenceInfo.NumMipLevels <= 0 || InSequenceInfo.Dim.X <= 0 || InSequenceInfo.Dim.Y <= 0)
	{
		return;
	}
	// FImgMediaLoader leaves TileSize=(0,0) for non-tiled sequences (DDS without tile metadata).
	// FMediaTileSelection::CreateForTargetMipLevel treats that sentinel as "single 1x1 tile";
	// substitute the full image dim here so the per-mip tile-coordinate math agrees and the
	// loop below ends up emitting exactly one (0, 0) tile per mip.
	const FIntPoint TileSize = (InSequenceInfo.TilingDescription.TileSize.X > 0 && InSequenceInfo.TilingDescription.TileSize.Y > 0)
		? InSequenceInfo.TilingDescription.TileSize
		: FIntPoint(FMath::Max(1, InSequenceInfo.Dim.X), FMath::Max(1, InSequenceInfo.Dim.Y));

	FUpdateInputs Snapshot;
	{
		FScopeLock Lock(&Mutex);
		Snapshot = Inputs;
	}

	// Pre-paint guard: emit nothing until the consumer has pushed real inputs. Falling back
	// to "full source" would spike-load mip 0 for one frame on open.
	const FVector2D RectSize = Snapshot.VisibleSourceRect.GetSize();
	if (RectSize.X <= 0.0 || RectSize.Y <= 0.0
		|| Snapshot.DisplayedSizePx.X <= 0.0 || Snapshot.DisplayedSizePx.Y <= 0.0)
	{
		return;
	}

	const int32 MaxMip = InSequenceInfo.NumMipLevels - 1;

	// Mip pair: load both floor(LOD) and ceil(LOD) so trilinear filtering has both samples
	// resident. LOD is taken along the higher-density axis (Min ratio). MipOverride pins to
	// a single mip when set. Shared with GetDisplayedMipFloor so the picker matches what's
	// actually loaded.
	int32 MipFloor = 0;
	int32 MipCeil = 0;
	if (!ComputeMipPair(Snapshot, MipFloor, MipCeil))
	{
		return;
	}
	MipFloor = FMath::Clamp(MipFloor, 0, MaxMip);
	MipCeil  = FMath::Clamp(MipCeil,  0, MaxMip);

	// Source-pixel rect to UV. Clamp guards against the consumer pushing an out-of-range rect.
	const FVector2D SourceDimD(InSequenceInfo.Dim);
	const FVector2D UvMin(
		FMath::Clamp(Snapshot.VisibleSourceRect.Min.X / SourceDimD.X, 0.0, 1.0),
		FMath::Clamp(Snapshot.VisibleSourceRect.Min.Y / SourceDimD.Y, 0.0, 1.0));
	const FVector2D UvMax(
		FMath::Clamp(Snapshot.VisibleSourceRect.Max.X / SourceDimD.X, 0.0, 1.0),
		FMath::Clamp(Snapshot.VisibleSourceRect.Max.Y / SourceDimD.Y, 0.0, 1.0));

	if (UvMin.X >= UvMax.X || UvMin.Y >= UvMax.Y)
	{
		return;
	}

	// Map UV window to tile coordinates at every mip in the [MipFloor, MipCeil] range.
	for (int32 Mip = MipFloor; Mip <= MipCeil; ++Mip)
	{
		const int32 MipDiv = 1 << Mip;
		const FIntPoint NumTilesAtMip(
			FMath::Max(1, FMath::CeilToInt32(double(InSequenceInfo.Dim.X) / MipDiv / TileSize.X)),
			FMath::Max(1, FMath::CeilToInt32(double(InSequenceInfo.Dim.Y) / MipDiv / TileSize.Y)));

		const int32 TileMinX = FMath::Clamp(FMath::FloorToInt32(UvMin.X * NumTilesAtMip.X), 0, NumTilesAtMip.X - 1);
		const int32 TileMinY = FMath::Clamp(FMath::FloorToInt32(UvMin.Y * NumTilesAtMip.Y), 0, NumTilesAtMip.Y - 1);
		const int32 TileMaxX = FMath::Clamp(FMath::CeilToInt32 (UvMax.X * NumTilesAtMip.X) - 1, 0, NumTilesAtMip.X - 1);
		const int32 TileMaxY = FMath::Clamp(FMath::CeilToInt32 (UvMax.Y * NumTilesAtMip.Y) - 1, 0, NumTilesAtMip.Y - 1);

		FMediaTileSelection* Selection = OutRequest.VisibleTiles.Find(Mip);
		if (Selection == nullptr)
		{
			Selection = &OutRequest.VisibleTiles.Add(Mip,
				FMediaTileSelection::CreateForTargetMipLevel(
					InSequenceInfo.Dim, TileSize, Mip, /*bDefaultVisibility*/ false));
		}
		for (int32 TileY = TileMinY; TileY <= TileMaxY; ++TileY)
		{
			for (int32 TileX = TileMinX; TileX <= TileMaxX; ++TileX)
			{
				Selection->SetVisible(TileX, TileY);
			}
		}
	}
}

} // namespace UE::MediaViewer::Private
