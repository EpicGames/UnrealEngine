// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Assets/MediaTileVisibility.h"
#include "HAL/CriticalSection.h"
#include "Math/Box2D.h"
#include "Math/Vector2D.h"
#include "Misc/Optional.h"

namespace UE::MediaViewer::Private
{

/** Tile-visibility provider that derives mip pair and tile coverage from a visible source rect plus its on-screen size. */
class FViewerTileVisibilityProvider final : public FMediaTileVisibilityProvider
{
public:
	/** Snapshot of the consumer-pushed viewport state consumed by the next GatherVisibleTiles call. */
	struct FUpdateInputs
	{
		/** Visible part of the source in source-pixel coordinates. Empty until first Update. */
		FBox2D VisibleSourceRect = FBox2D(ForceInit);

		/** Physical-pixel size of the area displaying VisibleSourceRect. Drives mip selection. */
		FVector2D DisplayedSizePx = FVector2D::ZeroVector;

		/** Optional pinned mip; overrides the LOD-derived mip pair when set. */
		TOptional<int32> MipOverride;
	};

	/** Replace the current viewport state used by the next GatherVisibleTiles call. thread safe. */
	void Update(const FUpdateInputs& InInputs);

	/**
	 * Returns the highest-resolution mip currently visible (floor of the LOD-derived mip,
	 * matching what GatherVisibleTiles loads). Returns INDEX_NONE before the first Update
	 * call or when the inputs are degenerate. MipOverride, when set, is returned directly.
	 *
	 * Used by the color picker to sample from the same mip the viewer is displaying -
	 * visibility-provider-driven sources only refresh visible mips, so reading mip 0
	 * would return stale or never-uploaded data.
	 */
	int32 GetDisplayedMipFloor() const;

	//~ Begin IMediaTileVisibilityProvider
	virtual void GatherVisibleTiles(const FMediaSequenceInfo& InSequenceInfo,
		FMediaTileVisibilityRequest& OutRequest) const override;
	virtual bool IsAlive() const override { return true; }
	//~ End IMediaTileVisibilityProvider

private:
	/** Guards Inputs against the cross-thread access between Update and GatherVisibleTiles. */
	mutable FCriticalSection Mutex;

	/** Latest viewport state pushed by the consumer. Default-constructed (empty) until first Update call. */
	FUpdateInputs Inputs;
};

} // namespace UE::MediaViewer::Private
