// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

#include "CQTest.h"

#include "CommonFrameRates.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/FrameRate.h"
#include "Misc/Guid.h"
#include "Misc/Paths.h"
#include "Widgets/SMediaViewerDropTarget.h"
#include "Widgets/SMediaViewerDropTargetInternal.h"

namespace UE::MediaViewer::Private::Tests
{

/** Creates an empty file at the given absolute path. Intermediate directories are created if missing. */
static void WriteTouchFile(const FString& InPath)
{
	FFileHelper::SaveStringToFile(FString(), *InPath);
}

} // namespace UE::MediaViewer::Private::Tests

// ---------------------------------------------------------------------------
// Fixture: GetFrameNumberAndStem
// ---------------------------------------------------------------------------

TEST_CLASS(FGetFrameNumberAndStemTests, "System.Plugins.MediaViewer.GetFrameNumberAndStem")
{
	TEST_METHOD(Simple_TrailingZeroPaddedNumber_ReturnsStemAndNumber)
	{
		using namespace UE::MediaViewer::Private;

		int32 FrameNumber = -9999;
		FString Stem = TEXT("UNTOUCHED");

		const bool bResult = GetFrameNumberAndStem(TEXT("frame_0042.png"), TEXT("png"), FrameNumber, Stem);

		ASSERT_THAT(IsTrue(bResult));
		ASSERT_THAT(AreEqual(42, FrameNumber));
		ASSERT_THAT(AreEqual(FString(TEXT("frame_")), Stem));
	}

	TEST_METHOD(DotSeparated_ExrSequence_ReturnsStemAndNumber)
	{
		using namespace UE::MediaViewer::Private;

		int32 FrameNumber = -9999;
		FString Stem = TEXT("UNTOUCHED");

		const bool bResult = GetFrameNumberAndStem(TEXT("render.0042.exr"), TEXT("exr"), FrameNumber, Stem);

		ASSERT_THAT(IsTrue(bResult));
		ASSERT_THAT(AreEqual(42, FrameNumber));
		ASSERT_THAT(AreEqual(FString(TEXT("render.")), Stem));
	}

	TEST_METHOD(NoLeadingZeros_SingleDigit_AcceptsAndReturnsStem)
	{
		using namespace UE::MediaViewer::Private;

		int32 FrameNumber = -9999;
		FString Stem = TEXT("UNTOUCHED");

		const bool bResult = GetFrameNumberAndStem(TEXT("frame_1.png"), TEXT("png"), FrameNumber, Stem);

		ASSERT_THAT(IsTrue(bResult));
		ASSERT_THAT(AreEqual(1, FrameNumber));
		ASSERT_THAT(AreEqual(FString(TEXT("frame_")), Stem));
	}

	TEST_METHOD(NoStem_AllDigits_ReturnsEmptyStem)
	{
		using namespace UE::MediaViewer::Private;

		int32 FrameNumber = -9999;
		FString Stem = TEXT("UNTOUCHED");

		const bool bResult = GetFrameNumberAndStem(TEXT("00000042.exr"), TEXT("exr"), FrameNumber, Stem);

		ASSERT_THAT(IsTrue(bResult));
		ASSERT_THAT(AreEqual(42, FrameNumber));
		ASSERT_THAT(AreEqual(FString(TEXT("")), Stem));
	}

	TEST_METHOD(AllZeros_ReturnsZeroFrameAndEmptyStem)
	{
		using namespace UE::MediaViewer::Private;

		int32 FrameNumber = -9999;
		FString Stem = TEXT("UNTOUCHED");

		const bool bResult = GetFrameNumberAndStem(TEXT("00000000000.exr"), TEXT("exr"), FrameNumber, Stem);

		ASSERT_THAT(IsTrue(bResult));
		ASSERT_THAT(AreEqual(0, FrameNumber));
		ASSERT_THAT(AreEqual(FString(TEXT("")), Stem));
	}

	TEST_METHOD(NoDigits_ReturnsFalse_OutParamsUnchanged)
	{
		using namespace UE::MediaViewer::Private;

		// Pre-set sentinels; on false path neither must be written.
		int32 FrameNumber = -9999;
		FString Stem = TEXT("UNTOUCHED");

		const bool bResult = GetFrameNumberAndStem(TEXT("frame.exr"), TEXT("exr"), FrameNumber, Stem);

		ASSERT_THAT(IsFalse(bResult));
		ASSERT_THAT(AreEqual(-9999, FrameNumber));
		ASSERT_THAT(AreEqual(FString(TEXT("UNTOUCHED")), Stem));
	}

	TEST_METHOD(EmbeddedNumber_LastRunOnly_ReturnsTailNumber)
	{
		using namespace UE::MediaViewer::Private;

		int32 FrameNumber = -9999;
		FString Stem = TEXT("UNTOUCHED");

		// Only the trailing digit run is extracted; embedded numbers stay in the stem.
		const bool bResult = GetFrameNumberAndStem(TEXT("shot_42_take_3.exr"), TEXT("exr"), FrameNumber, Stem);

		ASSERT_THAT(IsTrue(bResult));
		ASSERT_THAT(AreEqual(3, FrameNumber));
		ASSERT_THAT(AreEqual(FString(TEXT("shot_42_take_")), Stem));
	}

	// Validates the overflow guard: 11 significant digits exceed the int32 capacity ceiling.
	TEST_METHOD(OverflowEleven9s_ReturnsFalse_OutParamsUnchanged)
	{
		using namespace UE::MediaViewer::Private;

		int32 FrameNumber = -9999;
		FString Stem = TEXT("UNTOUCHED");

		const bool bResult = GetFrameNumberAndStem(TEXT("frame_99999999999.png"), TEXT("png"), FrameNumber, Stem);

		ASSERT_THAT(IsFalse(bResult));
		ASSERT_THAT(AreEqual(-9999, FrameNumber));
		ASSERT_THAT(AreEqual(FString(TEXT("UNTOUCHED")), Stem));
	}

	// Validates the leading-zero correction: 14 total chars but only 11 significant digits after
	// stripping leading zeros — must still be rejected as overflow.
	TEST_METHOD(OverflowWithLeadingZeros_StripLeadingZerosThenReject)
	{
		using namespace UE::MediaViewer::Private;

		int32 FrameNumber = -9999;
		FString Stem = TEXT("UNTOUCHED");

		const bool bResult = GetFrameNumberAndStem(TEXT("frame_0000099999999999.png"), TEXT("png"), FrameNumber, Stem);

		ASSERT_THAT(IsFalse(bResult));
		ASSERT_THAT(AreEqual(-9999, FrameNumber));
		ASSERT_THAT(AreEqual(FString(TEXT("UNTOUCHED")), Stem));
	}

	TEST_METHOD(MaxInt32_ExactBoundary_Accepted)
	{
		using namespace UE::MediaViewer::Private;

		int32 FrameNumber = -9999;
		FString Stem = TEXT("UNTOUCHED");

		const bool bResult = GetFrameNumberAndStem(TEXT("frame_2147483647.png"), TEXT("png"), FrameNumber, Stem);

		ASSERT_THAT(IsTrue(bResult));
		ASSERT_THAT(AreEqual(MAX_int32, FrameNumber));
		ASSERT_THAT(AreEqual(FString(TEXT("frame_")), Stem));
	}

	TEST_METHOD(AboveMaxInt32_OneBeyondBoundary_ReturnsFalse)
	{
		using namespace UE::MediaViewer::Private;

		int32 FrameNumber = -9999;
		FString Stem = TEXT("UNTOUCHED");

		// 2147483648 == MAX_int32 + 1; the range check must reject it.
		const bool bResult = GetFrameNumberAndStem(TEXT("frame_2147483648.png"), TEXT("png"), FrameNumber, Stem);

		ASSERT_THAT(IsFalse(bResult));
		ASSERT_THAT(AreEqual(-9999, FrameNumber));
		ASSERT_THAT(AreEqual(FString(TEXT("UNTOUCHED")), Stem));
	}

	// Validates the leading-zero correction for the acceptance path: 8 total chars but only
	// 2 significant digits — must not be rejected as overflow.
	TEST_METHOD(LeadingZerosWithSmallNumber_AcceptsNormalValue)
	{
		using namespace UE::MediaViewer::Private;

		int32 FrameNumber = -9999;
		FString Stem = TEXT("UNTOUCHED");

		const bool bResult = GetFrameNumberAndStem(TEXT("frame_00000042.png"), TEXT("png"), FrameNumber, Stem);

		ASSERT_THAT(IsTrue(bResult));
		ASSERT_THAT(AreEqual(42, FrameNumber));
		ASSERT_THAT(AreEqual(FString(TEXT("frame_")), Stem));
	}
};

// ---------------------------------------------------------------------------
// Fixture: FindClosestCommonFrameRate
// ---------------------------------------------------------------------------

TEST_CLASS(FFindClosestCommonFrameRateTests, "System.Plugins.MediaViewer.FindClosestCommonFrameRate")
{
	TEST_METHOD(Common24fps_ReturnsTwentyFour)
	{
		using namespace UE::MediaViewer::Private;

		const FFrameRate Result = FindClosestCommonFrameRate(24.0f);
		ASSERT_THAT(AreEqual(24, Result.Numerator));
		ASSERT_THAT(AreEqual(1, Result.Denominator));
	}

	TEST_METHOD(Common30fps_ReturnsThirty)
	{
		using namespace UE::MediaViewer::Private;

		const FFrameRate Result = FindClosestCommonFrameRate(30.0f);
		ASSERT_THAT(AreEqual(30, Result.Numerator));
		ASSERT_THAT(AreEqual(1, Result.Denominator));
	}

	// 23.976 fps is NTSC_24: FFrameRate(24000, 1001)
	TEST_METHOD(NTSC2398_ReturnsNTSC24)
	{
		using namespace UE::MediaViewer::Private;

		const FFrameRate Result = FindClosestCommonFrameRate(23.976f);
		ASSERT_THAT(AreEqual(24000, Result.Numerator));
		ASSERT_THAT(AreEqual(1001, Result.Denominator));
	}

	// 29.97 fps is NTSC_30: FFrameRate(30000, 1001)
	TEST_METHOD(NTSC2997_ReturnsNTSC30)
	{
		using namespace UE::MediaViewer::Private;

		const FFrameRate Result = FindClosestCommonFrameRate(29.97f);
		ASSERT_THAT(AreEqual(30000, Result.Numerator));
		ASSERT_THAT(AreEqual(1001, Result.Denominator));
	}

	TEST_METHOD(ZeroInput_FallsBackTo24fps)
	{
		using namespace UE::MediaViewer::Private;

		const FFrameRate Result = FindClosestCommonFrameRate(0.0f);
		ASSERT_THAT(AreEqual(24, Result.Numerator));
		ASSERT_THAT(AreEqual(1, Result.Denominator));
	}

	TEST_METHOD(NegativeInput_FallsBackTo24fps)
	{
		using namespace UE::MediaViewer::Private;

		const FFrameRate Result = FindClosestCommonFrameRate(-1.0f);
		ASSERT_THAT(AreEqual(24, Result.Numerator));
		ASSERT_THAT(AreEqual(1, Result.Denominator));
	}
};

// ---------------------------------------------------------------------------
// Fixture: DetectImageSequence (filesystem-touching)
// ---------------------------------------------------------------------------

TEST_CLASS(FDetectImageSequenceTests, "System.Plugins.MediaViewer.DetectImageSequence")
{
	/** Absolute path to the unique temp directory for the current test method. */
	FString TempDir;

	BEFORE_EACH()
	{
		// Use a GUID-named subdirectory so parallel test runs don't collide.
		TempDir = FPaths::Combine(FPaths::AutomationTransientDir(),
			TEXT("SMediaViewerDropTargetTests"),
			FGuid::NewGuid().ToString());
		IFileManager::Get().MakeDirectory(*TempDir, /*Tree=*/true);
	}

	AFTER_EACH()
	{
		IFileManager::Get().DeleteDirectory(*TempDir, /*RequireExists=*/false, /*Tree=*/true);
	}

	TEST_METHOD(ValidSequence_TwoFiles_DetectsSequenceAndReturnsFrameNumbers)
	{
		using namespace UE::MediaViewer::Private::Tests;

		WriteTouchFile(FPaths::Combine(TempDir, TEXT("frame_0001.png")));
		WriteTouchFile(FPaths::Combine(TempDir, TEXT("frame_0002.png")));

		const FString DroppedFile = FPaths::Combine(TempDir, TEXT("frame_0001.png"));
		int32 DroppedFrame = -9999;
		int32 FirstFrame = -9999;

		const bool bResult = UE::MediaViewer::Private::DetectImageSequence(
			DroppedFile, DroppedFrame, FirstFrame);

		ASSERT_THAT(IsTrue(bResult));
		ASSERT_THAT(AreEqual(1, DroppedFrame));
		ASSERT_THAT(AreEqual(1, FirstFrame));
	}

	TEST_METHOD(ValidSequence_DroppedMiddleFrame_ReturnsCorrectFirstFrame)
	{
		using namespace UE::MediaViewer::Private::Tests;

		WriteTouchFile(FPaths::Combine(TempDir, TEXT("seq.0001.exr")));
		WriteTouchFile(FPaths::Combine(TempDir, TEXT("seq.0002.exr")));
		WriteTouchFile(FPaths::Combine(TempDir, TEXT("seq.0003.exr")));

		const FString DroppedFile = FPaths::Combine(TempDir, TEXT("seq.0002.exr"));
		int32 DroppedFrame = -9999;
		int32 FirstFrame = -9999;

		const bool bResult = UE::MediaViewer::Private::DetectImageSequence(
			DroppedFile, DroppedFrame, FirstFrame);

		ASSERT_THAT(IsTrue(bResult));
		ASSERT_THAT(AreEqual(2, DroppedFrame));
		ASSERT_THAT(AreEqual(1, FirstFrame));
	}

	// UE-373622 repro: a lone image with a trailing number must not be identified as a sequence.
	TEST_METHOD(LoneFile_NoSiblings_ReturnsFalse)
	{
		using namespace UE::MediaViewer::Private::Tests;

		WriteTouchFile(FPaths::Combine(TempDir, TEXT("IMG_2033.jpg")));

		const FString DroppedFile = FPaths::Combine(TempDir, TEXT("IMG_2033.jpg"));
		int32 DroppedFrame = -9999;
		int32 FirstFrame = -9999;

		const bool bResult = UE::MediaViewer::Private::DetectImageSequence(
			DroppedFile, DroppedFrame, FirstFrame);

		ASSERT_THAT(IsFalse(bResult));
	}

	// Strict mode: a sibling with a different stem disqualifies the whole directory.
	TEST_METHOD(StraySibling_DifferentStem_ReturnsFalse)
	{
		using namespace UE::MediaViewer::Private::Tests;

		WriteTouchFile(FPaths::Combine(TempDir, TEXT("frame_0001.png")));
		WriteTouchFile(FPaths::Combine(TempDir, TEXT("frame_0002.png")));
		WriteTouchFile(FPaths::Combine(TempDir, TEXT("final_0001.png")));

		const FString DroppedFile = FPaths::Combine(TempDir, TEXT("frame_0001.png"));
		int32 DroppedFrame = -9999;
		int32 FirstFrame = -9999;

		const bool bResult = UE::MediaViewer::Private::DetectImageSequence(
			DroppedFile, DroppedFrame, FirstFrame);

		ASSERT_THAT(IsFalse(bResult));
	}

	// Strict mode: a sibling without a trailing frame number disqualifies the whole directory.
	TEST_METHOD(StraySibling_NoFrameNumber_ReturnsFalse)
	{
		using namespace UE::MediaViewer::Private::Tests;

		WriteTouchFile(FPaths::Combine(TempDir, TEXT("frame_0001.png")));
		WriteTouchFile(FPaths::Combine(TempDir, TEXT("frame_0002.png")));
		WriteTouchFile(FPaths::Combine(TempDir, TEXT("notes.png")));

		const FString DroppedFile = FPaths::Combine(TempDir, TEXT("frame_0001.png"));
		int32 DroppedFrame = -9999;
		int32 FirstFrame = -9999;

		const bool bResult = UE::MediaViewer::Private::DetectImageSequence(
			DroppedFile, DroppedFrame, FirstFrame);

		ASSERT_THAT(IsFalse(bResult));
	}

	// When DetectImageSequence returns false, out-params must not be mutated.
	TEST_METHOD(OutParams_NotMutatedOnFalse_SentinelsUnchanged)
	{
		using namespace UE::MediaViewer::Private::Tests;

		// Same setup as LoneFile_NoSiblings — guaranteed to return false.
		WriteTouchFile(FPaths::Combine(TempDir, TEXT("IMG_2033.jpg")));

		const FString DroppedFile = FPaths::Combine(TempDir, TEXT("IMG_2033.jpg"));
		int32 DroppedFrame = -9999;
		int32 FirstFrame = -9999;

		const bool bResult = UE::MediaViewer::Private::DetectImageSequence(
			DroppedFile, DroppedFrame, FirstFrame);

		ASSERT_THAT(IsFalse(bResult));
		ASSERT_THAT(AreEqual(-9999, DroppedFrame));
		ASSERT_THAT(AreEqual(-9999, FirstFrame));
	}
};

#endif // WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR
