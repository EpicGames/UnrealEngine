// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

#include "CQTest.h"

#include "Assets/MediaSequenceInfo.h"
#include "Assets/MediaTileSelection.h"
#include "Assets/MediaTileVisibility.h"
#include "Providers/ViewerTileVisibilityProvider.h"

namespace UE::MediaViewer::Private::Tests
{

/**
 * Build a 1024x1024, 4-mip, 256x256-tile FMediaSequenceInfo (4x4 tiles at mip 0).
 * This is the baseline geometry used by every test in this fixture.
 */
static FMediaSequenceInfo MakeSequenceInfo()
{
	FMediaSequenceInfo Info;
	Info.Name          = TEXT("TestSeq");
	Info.Dim           = FIntPoint(1024, 1024);
	Info.NumMipLevels  = 4;
	Info.TilingDescription.TileSize    = FIntPoint(256, 256);
	Info.TilingDescription.TileNum     = FIntPoint(4, 4);
	Info.TilingDescription.TileBorderSize = 0;
	return Info;
}

/**
 * Build a provider with the given Update inputs already applied. Returns a thread-safe
 * shared pointer to mirror production usage (see MediaSourceImageViewer.cpp).
 */
static TSharedPtr<FViewerTileVisibilityProvider, ESPMode::ThreadSafe> MakeProvider(
	const FBox2D& InRect,
	const FVector2D& InDisplayedSizePx,
	TOptional<int32> InMipOverride = TOptional<int32>())
{
	TSharedPtr<FViewerTileVisibilityProvider, ESPMode::ThreadSafe> Provider =
		MakeShared<FViewerTileVisibilityProvider, ESPMode::ThreadSafe>();

	FViewerTileVisibilityProvider::FUpdateInputs Inputs;
	Inputs.VisibleSourceRect = InRect;
	Inputs.DisplayedSizePx   = InDisplayedSizePx;
	Inputs.MipOverride       = InMipOverride;
	Provider->Update(Inputs);

	return Provider;
}

} // namespace UE::MediaViewer::Private::Tests

// ---------------------------------------------------------------------------
// Test fixture
// ---------------------------------------------------------------------------

TEST_CLASS(FViewerTileVisibilityProviderTests, "System.Plugins.MediaViewer.ViewerTileVisibilityProvider")
{
	// Convenience aliases so test bodies remain readable.
	using FProvider    = UE::MediaViewer::Private::FViewerTileVisibilityProvider;
	using FProviderPtr = TSharedPtr<FProvider, ESPMode::ThreadSafe>;

	// -----------------------------------------------------------------------
	// Scenario 1 — One-to-one zoom (H1 epsilon-guard regression)
	// Rect == source, displayed == source => Ratio exactly 1.0; the epsilon
	// guard must keep this in the Ratio <= 1 branch so only mip 0 is emitted.
	// -----------------------------------------------------------------------

	TEST_METHOD(OneToOneZoom_OnlyMip0_AllSixteenTilesVisible)
	{
		using namespace UE::MediaViewer::Private::Tests;

		FProviderPtr Provider = MakeProvider(
			FBox2D(FVector2D(0.0, 0.0), FVector2D(1024.0, 1024.0)),
			FVector2D(1024.0, 1024.0));

		const FMediaSequenceInfo SeqInfo = MakeSequenceInfo();
		FMediaTileVisibilityRequest OutRequest;
		Provider->GatherVisibleTiles(SeqInfo, OutRequest);

		// Only mip 0 must be present — the H1 regression produced mip 1 as well.
		ASSERT_THAT(AreEqual(1, OutRequest.VisibleTiles.Num()));
		ASSERT_THAT(IsTrue(OutRequest.VisibleTiles.Contains(0)));
		ASSERT_THAT(IsFalse(OutRequest.VisibleTiles.Contains(1)));

		const FMediaTileSelection& Sel = OutRequest.VisibleTiles[0];
		ASSERT_THAT(AreEqual(16, Sel.NumVisibleTiles()));
	}

	// -----------------------------------------------------------------------
	// Scenario 2 — Zoom-in (display larger than source region)
	// Visible rect is 256x256 out of 1024x1024, displayed at 1024x1024.
	// Ratio = 256/1024 = 0.25 < 1 => mip 0 only.
	// UV window 0.25..0.5 maps to tile (1,1) at mip 0.
	// -----------------------------------------------------------------------

	TEST_METHOD(ZoomIn_OnlyMip0_SingleTile1x1Visible)
	{
		using namespace UE::MediaViewer::Private::Tests;

		FProviderPtr Provider = MakeProvider(
			FBox2D(FVector2D(256.0, 256.0), FVector2D(512.0, 512.0)),
			FVector2D(1024.0, 1024.0));

		const FMediaSequenceInfo SeqInfo = MakeSequenceInfo();
		FMediaTileVisibilityRequest OutRequest;
		Provider->GatherVisibleTiles(SeqInfo, OutRequest);

		ASSERT_THAT(AreEqual(1, OutRequest.VisibleTiles.Num()));
		ASSERT_THAT(IsTrue(OutRequest.VisibleTiles.Contains(0)));

		const FMediaTileSelection& Sel = OutRequest.VisibleTiles[0];
		ASSERT_THAT(AreEqual(1, Sel.NumVisibleTiles()));
		ASSERT_THAT(IsTrue(Sel.IsVisible(1, 1)));
		// Confirm neighbouring tiles are not lit.
		ASSERT_THAT(IsFalse(Sel.IsVisible(0, 0)));
		ASSERT_THAT(IsFalse(Sel.IsVisible(2, 2)));
	}

	// -----------------------------------------------------------------------
	// Scenario 3 — Zoom-out exact integer LOD
	// Rect 1024x1024, displayed 256x256. Ratio = 4, log2 = 2 exactly.
	// floor == ceil == 2 => only mip 2 populated.
	// -----------------------------------------------------------------------

	TEST_METHOD(ZoomOutExactIntegerLOD_OnlyMip2)
	{
		using namespace UE::MediaViewer::Private::Tests;

		FProviderPtr Provider = MakeProvider(
			FBox2D(FVector2D(0.0, 0.0), FVector2D(1024.0, 1024.0)),
			FVector2D(256.0, 256.0));

		const FMediaSequenceInfo SeqInfo = MakeSequenceInfo();
		FMediaTileVisibilityRequest OutRequest;
		Provider->GatherVisibleTiles(SeqInfo, OutRequest);

		ASSERT_THAT(AreEqual(1, OutRequest.VisibleTiles.Num()));
		ASSERT_THAT(IsTrue(OutRequest.VisibleTiles.Contains(2)));
		ASSERT_THAT(IsFalse(OutRequest.VisibleTiles.Contains(1)));
		ASSERT_THAT(IsFalse(OutRequest.VisibleTiles.Contains(3)));
	}

	// -----------------------------------------------------------------------
	// Scenario 4 — Zoom-out non-integer LOD (mip pair)
	// Rect 1024x1024, displayed 384x384. Ratio = 1024/384 ≈ 2.667, log2 ≈ 1.415.
	// floor = 1, ceil = 2 => both mips 1 and 2 must be in VisibleTiles.
	// -----------------------------------------------------------------------

	TEST_METHOD(ZoomOutNonIntegerLOD_MipPair1And2)
	{
		using namespace UE::MediaViewer::Private::Tests;

		FProviderPtr Provider = MakeProvider(
			FBox2D(FVector2D(0.0, 0.0), FVector2D(1024.0, 1024.0)),
			FVector2D(384.0, 384.0));

		const FMediaSequenceInfo SeqInfo = MakeSequenceInfo();
		FMediaTileVisibilityRequest OutRequest;
		Provider->GatherVisibleTiles(SeqInfo, OutRequest);

		ASSERT_THAT(AreEqual(2, OutRequest.VisibleTiles.Num()));
		ASSERT_THAT(IsTrue(OutRequest.VisibleTiles.Contains(1)));
		ASSERT_THAT(IsTrue(OutRequest.VisibleTiles.Contains(2)));
	}

	// -----------------------------------------------------------------------
	// Scenario 5a — Mip override pin (in-range value)
	// MipOverride = 3 must override LOD calculation; only mip 3 in output.
	// -----------------------------------------------------------------------

	TEST_METHOD(MipOverride_PinsToSpecifiedMip)
	{
		using namespace UE::MediaViewer::Private::Tests;

		// The zoom ratio here would normally pick mip 0; override must win.
		FProviderPtr Provider = MakeProvider(
			FBox2D(FVector2D(0.0, 0.0), FVector2D(1024.0, 1024.0)),
			FVector2D(1024.0, 1024.0),
			TOptional<int32>(3));

		const FMediaSequenceInfo SeqInfo = MakeSequenceInfo();
		FMediaTileVisibilityRequest OutRequest;
		Provider->GatherVisibleTiles(SeqInfo, OutRequest);

		ASSERT_THAT(AreEqual(1, OutRequest.VisibleTiles.Num()));
		ASSERT_THAT(IsTrue(OutRequest.VisibleTiles.Contains(3)));
		ASSERT_THAT(IsFalse(OutRequest.VisibleTiles.Contains(0)));
	}

	// -----------------------------------------------------------------------
	// Scenario 5b — Mip override out-of-range clamped to MaxMip
	// MaxMip = NumMipLevels - 1 = 3; override 99 must clamp to 3.
	// -----------------------------------------------------------------------

	TEST_METHOD(MipOverride_OutOfRangeClampedToMaxMip)
	{
		using namespace UE::MediaViewer::Private::Tests;

		FProviderPtr Provider = MakeProvider(
			FBox2D(FVector2D(0.0, 0.0), FVector2D(1024.0, 1024.0)),
			FVector2D(1024.0, 1024.0),
			TOptional<int32>(99));

		const FMediaSequenceInfo SeqInfo = MakeSequenceInfo();
		FMediaTileVisibilityRequest OutRequest;
		Provider->GatherVisibleTiles(SeqInfo, OutRequest);

		// NumMipLevels = 4, so MaxMip = 3.
		ASSERT_THAT(AreEqual(1, OutRequest.VisibleTiles.Num()));
		ASSERT_THAT(IsTrue(OutRequest.VisibleTiles.Contains(3)));
	}

	// -----------------------------------------------------------------------
	// Scenario 6 — Clamped UV (rect extends past source)
	// Rect (-128,-128)..(2048,2048) is RectSize 2176x2176; UV must clamp to [0,1]
	// without crashing or under-counting tiles. DisplayedSizePx is set to match
	// RectSize so Ratio = 1.0 (mip 0 only), isolating the UV-clamp behavior from
	// the LOD math already covered by the zoom-out tests above.
	// -----------------------------------------------------------------------

	TEST_METHOD(ClampedUV_RectBeyondSourceEdges_AllTilesVisible)
	{
		using namespace UE::MediaViewer::Private::Tests;

		FProviderPtr Provider = MakeProvider(
			FBox2D(FVector2D(-128.0, -128.0), FVector2D(2048.0, 2048.0)),
			FVector2D(2176.0, 2176.0));

		const FMediaSequenceInfo SeqInfo = MakeSequenceInfo();
		FMediaTileVisibilityRequest OutRequest;
		Provider->GatherVisibleTiles(SeqInfo, OutRequest);

		ASSERT_THAT(IsTrue(OutRequest.VisibleTiles.Contains(0)));
		const FMediaTileSelection& Sel = OutRequest.VisibleTiles[0];
		ASSERT_THAT(AreEqual(16, Sel.NumVisibleTiles()));
	}

	// -----------------------------------------------------------------------
	// Scenario 7 — Truly degenerate collapsed rect (equal Min and Max)
	// RectSize is (0,0) => pre-paint guard fires => no output.
	// -----------------------------------------------------------------------

	TEST_METHOD(CollapsedRect_ZeroSize_ProducesNoOutput)
	{
		using namespace UE::MediaViewer::Private::Tests;

		FProviderPtr Provider = MakeProvider(
			FBox2D(FVector2D(512.0, 512.0), FVector2D(512.0, 512.0)),
			FVector2D(1024.0, 1024.0));

		const FMediaSequenceInfo SeqInfo = MakeSequenceInfo();
		FMediaTileVisibilityRequest OutRequest;
		Provider->GatherVisibleTiles(SeqInfo, OutRequest);

		ASSERT_THAT(AreEqual(0, OutRequest.VisibleTiles.Num()));
	}

	// -----------------------------------------------------------------------
	// Scenario 8 — Pre-paint guard: no Update called
	// Default-constructed Inputs has VisibleSourceRect == empty FBox2D(ForceInit)
	// (zero size) and DisplayedSizePx == ZeroVector; both trigger the guard.
	// -----------------------------------------------------------------------

	TEST_METHOD(NeverUpdated_ProducesNoOutput)
	{
		// No Update call — exercises the default-constructed FUpdateInputs guard.
		FProviderPtr Provider = MakeShared<FProvider, ESPMode::ThreadSafe>();

		const FMediaSequenceInfo SeqInfo = UE::MediaViewer::Private::Tests::MakeSequenceInfo();
		FMediaTileVisibilityRequest OutRequest;
		Provider->GatherVisibleTiles(SeqInfo, OutRequest);

		ASSERT_THAT(AreEqual(0, OutRequest.VisibleTiles.Num()));
	}

	// -----------------------------------------------------------------------
	// Scenario 9 — Pre-paint guard: zero DisplayedSizePx
	// -----------------------------------------------------------------------

	TEST_METHOD(ZeroDisplayedSize_ProducesNoOutput)
	{
		using namespace UE::MediaViewer::Private::Tests;

		FProviderPtr Provider = MakeProvider(
			FBox2D(FVector2D(0.0, 0.0), FVector2D(1024.0, 1024.0)),
			FVector2D(0.0, 0.0));

		const FMediaSequenceInfo SeqInfo = MakeSequenceInfo();
		FMediaTileVisibilityRequest OutRequest;
		Provider->GatherVisibleTiles(SeqInfo, OutRequest);

		ASSERT_THAT(AreEqual(0, OutRequest.VisibleTiles.Num()));
	}

	// -----------------------------------------------------------------------
	// Scenario 10 — OutRequest preservation
	// Pre-populate VisibleTiles[5] with a foreign selection; then run a
	// normal gather that produces mip 0.  The mip-5 entry must survive
	// untouched (the interface contract forbids clearing OutRequest).
	//
	// The second part checks OR semantics: pre-populate mip 0 with a single
	// tile at (3,3) marked visible, then run a gather covering only tile (0,0).
	// The provider's Find-then-AddIfAbsent path reuses the existing selection,
	// so after gather both (3,3) and (0,0) must be visible.
	// -----------------------------------------------------------------------

	TEST_METHOD(OutRequest_ForeignMipEntryPreservedAfterGather)
	{
		using namespace UE::MediaViewer::Private::Tests;

		const FMediaSequenceInfo SeqInfo = MakeSequenceInfo();

		// Seed a mip-5 entry (outside the sequence's actual mip range) to act as
		// a sentinel that must survive the gather call.
		FMediaTileVisibilityRequest OutRequest;
		OutRequest.VisibleTiles.Add(5, FMediaTileSelection(2, 2, false));

		FProviderPtr Provider = MakeProvider(
			FBox2D(FVector2D(0.0, 0.0), FVector2D(1024.0, 1024.0)),
			FVector2D(1024.0, 1024.0));
		Provider->GatherVisibleTiles(SeqInfo, OutRequest);

		// Mip 0 must have been added by the gather.
		ASSERT_THAT(IsTrue(OutRequest.VisibleTiles.Contains(0)));
		// Mip 5 sentinel must still be present (provider must not call Reset / clear map).
		ASSERT_THAT(IsTrue(OutRequest.VisibleTiles.Contains(5)));
	}

	TEST_METHOD(OutRequest_ExistingMip0EntryIsReusedWithOrSemantics)
	{
		using namespace UE::MediaViewer::Private::Tests;

		const FMediaSequenceInfo SeqInfo = MakeSequenceInfo();

		// Pre-seed mip 0 with tile (3,3) visible (bottom-right corner).
		FMediaTileVisibilityRequest OutRequest;
		FMediaTileSelection PreSeed = FMediaTileSelection::CreateForTargetMipLevel(
			SeqInfo.Dim, SeqInfo.TilingDescription.TileSize, 0, false);
		PreSeed.SetVisible(3, 3);
		OutRequest.VisibleTiles.Add(0, MoveTemp(PreSeed));

		// Gather with a zoomed-in rect covering only tile (0,0) at UV 0..0.25.
		FProviderPtr Provider = MakeProvider(
			FBox2D(FVector2D(0.0, 0.0), FVector2D(256.0, 256.0)),
			FVector2D(1024.0, 1024.0));
		Provider->GatherVisibleTiles(SeqInfo, OutRequest);

		ASSERT_THAT(IsTrue(OutRequest.VisibleTiles.Contains(0)));
		const FMediaTileSelection& Sel = OutRequest.VisibleTiles[0];

		// The provider must have reused (not replaced) the existing selection,
		// so the pre-seeded tile (3,3) must still be marked visible.
		ASSERT_THAT(IsTrue(Sel.IsVisible(3, 3)));
		// And the newly gathered tile (0,0) must also be visible.
		ASSERT_THAT(IsTrue(Sel.IsVisible(0, 0)));
	}
};

#endif // WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR
