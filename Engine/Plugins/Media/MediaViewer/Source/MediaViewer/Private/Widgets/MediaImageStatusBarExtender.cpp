// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/MediaImageStatusBarExtender.h"

namespace UE::MediaViewer
{

void FMediaImageStatusBarExtender::AddExtension(FName InExtensionHook, EExtensionHook::Position InHookPosition,
	const TSharedPtr<FUICommandList>& InCommands, const FMediaImageStatusBarExtension::FDelegate& InDelegate)
{
	FMediaImageStatusBarExtension& Extension = Extensions.AddDefaulted_GetRef();
	Extension.Hook = InExtensionHook;
	Extension.HookPosition = InHookPosition;
	Extension.CommandList = InCommands;
	Extension.Delegate = InDelegate;
}

void FMediaImageStatusBarExtender::Apply(FName InExtensionHook, EExtensionHook::Position InHookPosition,
	const TSharedRef<SHorizontalBox>& InHorizontalBox) const
{
	for (const FMediaImageStatusBarExtension& Extension : Extensions)
	{
		if (Extension.Hook == InExtensionHook && Extension.HookPosition == InHookPosition)
		{
			Extension.Delegate.ExecuteIfBound(InHorizontalBox);
		}
	}
}

} // UE::MediaViewer
