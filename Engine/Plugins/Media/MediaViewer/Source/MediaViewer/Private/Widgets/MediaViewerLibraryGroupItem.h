// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Views/TableViewTypeTraits.h"
#include "Library/IMediaViewerLibrary.h"

template <>
struct TIsValidListItem<UE::MediaViewer::IMediaViewerLibrary::FGroupItem>
{
	enum
	{
		Value = true
	};
};

/**
 * Functionality (e.g. setting the value, testing invalid value) specialized for FGroupItem.
 */
template <>
struct TListTypeTraits<UE::MediaViewer::IMediaViewerLibrary::FGroupItem>
{
public:
	typedef UE::MediaViewer::IMediaViewerLibrary::FGroupItem NullableType;

	using MapKeyFuncs = TDefaultMapHashableKeyFuncs<UE::MediaViewer::IMediaViewerLibrary::FGroupItem, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<UE::MediaViewer::IMediaViewerLibrary::FGroupItem, FSparseItemInfo, false>;
	using SetKeyFuncs = DefaultKeyFuncs<UE::MediaViewer::IMediaViewerLibrary::FGroupItem>;

	template<typename U>
	static void AddReferencedObjects(FReferenceCollector& InCollector,
		TArray<UE::MediaViewer::IMediaViewerLibrary::FGroupItem>& ItemsWithGeneratedWidgets,
		TSet<UE::MediaViewer::IMediaViewerLibrary::FGroupItem>& SelectedItems,
		TMap<const U*, UE::MediaViewer::IMediaViewerLibrary::FGroupItem>& WidgetToItemMap)
	{
	}

	static bool IsPtrValid(const UE::MediaViewer::IMediaViewerLibrary::FGroupItem& InValue)
	{
		return InValue.GroupId.IsValid();
	}

	static void ResetPtr(UE::MediaViewer::IMediaViewerLibrary::FGroupItem& InValue)
	{		
		InValue.GroupId = FGuid();
		InValue.ItemId = FGuid();
	}

	static UE::MediaViewer::IMediaViewerLibrary::FGroupItem MakeNullPtr()
	{
		return UE::MediaViewer::IMediaViewerLibrary::FGroupItem(FGuid(), FGuid());
	}

	static UE::MediaViewer::IMediaViewerLibrary::FGroupItem NullableItemTypeConvertToItemType(const UE::MediaViewer::IMediaViewerLibrary::FGroupItem& InValue)
	{
		return InValue;
	}

	static FString DebugDump(UE::MediaViewer::IMediaViewerLibrary::FGroupItem InValue)
	{
		return InValue.GroupId.ToString() + TEXT(": ") + InValue.ItemId.ToString();
	}

	class SerializerType
	{
	};
};
