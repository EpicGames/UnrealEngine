// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/MediaViewerLibraryItemAssetDragDropOperation.h"

#include "DetailLayoutBuilder.h"
#include "Library/MediaViewerLibrary.h"
#include "Library/MediaViewerLibraryGroup.h"
#include "Library/MediaViewerLibraryItem.h"
#include "Math/Color.h"
#include "MediaViewerLibraryItemAssetDragDropOperation.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "MediaViewerLibraryItemAssetDragDropOperation"

namespace UE::MediaViewer::Private
{

FMediaViewerLibraryItemAssetDragDropOperation::FMediaViewerLibraryItemAssetDragDropOperation(const TSharedRef<IMediaViewerLibrary>& InLibrary,
	const IMediaViewerLibrary::FGroupItem& InGroupItem)
	: GroupItem(InGroupItem)
	, Decorator(SNew(SBorder))
{
	if (TSharedPtr<FMediaViewerLibraryItem> Item = InLibrary->GetItem(InGroupItem.ItemId))
	{
		if (TOptional<FAssetData> ItemAssetData = Item->AsAsset())
		{
			// Initializes the base asset drop operation
			Init({ItemAssetData.GetValue()}, {}, nullptr);
		}

		CreateDecorator(Item);
	}
	
	Construct();
}

const IMediaViewerLibrary::FGroupItem& FMediaViewerLibraryItemAssetDragDropOperation::GetGroupItem() const
{
	return GroupItem;
}

TSharedPtr<SWidget> FMediaViewerLibraryItemAssetDragDropOperation::GetDefaultDecorator() const
{
	return Decorator;
}

FCursorReply FMediaViewerLibraryItemAssetDragDropOperation::OnCursorQuery()
{
	return FCursorReply::Cursor(EMouseCursor::GrabHandClosed);
}

void FMediaViewerLibraryItemAssetDragDropOperation::CreateDecorator(const TSharedPtr<FMediaViewerLibraryItem>& InItem)
{
	if (!InItem.IsValid())
	{
		return;
	}

	ThumbnailBrush = InItem->CreateThumbnail();

	Decorator->SetContent(
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5.f, 5.f, 5.f, 5.f)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(ThumbnailBrush.IsValid() ? ThumbnailBrush.Get() : nullptr)
			.DesiredSizeOverride(FVector2D(24.f))
		] 
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.Padding(3.f, 5.f, 5.f, 5.f)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(InItem->Name)
				.ToolTipText(InItem->ToolTip)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(InItem->GetItemTypeDisplayName())
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		]
	);
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
