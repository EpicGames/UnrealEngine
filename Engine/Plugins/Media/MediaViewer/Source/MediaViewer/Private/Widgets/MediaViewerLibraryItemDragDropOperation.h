// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/DecoratedDragDropOp.h"

#include "Library/IMediaViewerLibrary.h"
#include "Templates/SharedPointer.h"

class SBorder;
enum class EItemDropZone;
struct FMediaViewerLibraryEntry;
struct FMediaViewerLibraryGroup;
struct FMediaViewerLibraryItem;
struct FSlateBrush;

namespace UE::MediaViewer
{
	enum class EMediaImageViewerPosition : uint8;
}

namespace UE::MediaViewer::Private
{

class FMediaViewerLibraryItemDragDropOperation : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FMediaViewerLibraryItemDragDropOperation, FDecoratedDragDropOp)

	FMediaViewerLibraryItemDragDropOperation(const TSharedRef<IMediaViewerLibrary>& InLibrary,
		const IMediaViewerLibrary::FGroupItem& InGroupItem);
	
	virtual ~FMediaViewerLibraryItemDragDropOperation() override = default;

	const IMediaViewerLibrary::FGroupItem& GetGroupItem() const;

	//~ Begin FDragDropOp
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
	virtual FCursorReply OnCursorQuery() override;
	//~ End FDragDropOp

protected:
	IMediaViewerLibrary::FGroupItem GroupItem;
	TSharedPtr<FSlateBrush> ThumbnailBrush;
	TSharedRef<SBorder> Decorator;

	void CreateDecorator(const TSharedPtr<FMediaViewerLibraryItem>& InItem);
};

} // UE::MediaViewer::Private
