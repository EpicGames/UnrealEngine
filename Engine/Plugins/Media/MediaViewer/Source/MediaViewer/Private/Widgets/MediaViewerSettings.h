// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/EnumAsByte.h"
#include "Math/Color.h"
#include "Math/Vector.h"
#include "Styling/StyleColors.h"
#include "Types/SlateEnums.h"
#include "UObject/ObjectPtr.h"

#include "MediaViewerSettings.generated.h"

class UTexture2D;

UENUM()
enum class EMediaViewerMediaSyncType : uint8
{
	/** The offset is relative to the start time on both videos. A scrub change of 2s in video A will move video B also by 2s. */
	RelativeTime,

	/** The offset is relative to slider value on both videos. A scrub change of 10% in video A will move video B by 10%. */
	RelativeProgress,

	/** Time is synced exactly between videos. Video A changing from 5s to 10s will change video B from *anything* to 10s. */
	AbsoluteTime,

	/** Video progress is synced exactly between videos. Video A changing from 10% to 30% will change video B from *anything* to 30%. */
	AbsoluteProgress
};

/** Settings for the viewer. */
USTRUCT()
struct FMediaViewerSettings
{
	GENERATED_BODY()

	/** Color painted to the viewer behind the images. */
	UPROPERTY(EditAnywhere, Config, Category = "Background")
	FLinearColor ClearColor = FStyleColors::Black.GetSpecifiedColor();

	/** Texture tiled behind the images, on top of the clear color. */
	UPROPERTY(EditAnywhere, Config, Category = "Background")
	TSoftObjectPtr<UTexture2D> Texture = nullptr;

	/** Offset of the tiled background texture. */
	UPROPERTY(EditAnywhere, Config, Category = "Background", meta = (EditCondition = "Texture != nullptr"))
	FVector2D Offset = FVector2D::ZeroVector;

	/** Scale of the tiled background texture. */
	UPROPERTY(EditAnywhere, Config, Category = "Background", meta = (UIMin = 0.1, ClampMin = 0.1, UIMax = 10, ClampMax = 10,
		EditCondition = "Texture != nullptr"))
	float Scale = 1.f;

	/** If true, when one image transform changes, the other image transforms will change an equal amount. */
	UPROPERTY(EditAnywhere, Config, Category = "AB View", DisplayName = "Lock Transforms")
	bool bAreTransformsLocked = true;

	/** When in AB view, whether the viewers are split horizontally or vertically. */
	UPROPERTY(Config)
	TEnumAsByte<EOrientation> ABOrientation = Orient_Horizontal;

	/** Splitter location (0-100) */
	UPROPERTY(EditAnywhere, Config, Category = "AB View", DisplayName = "AB Splitter Location",
		meta = (UIMin = 0, ClampMin = 0, UIMax = 100, ClampMax = 100, Units = "Percent"))
	float ABSplitterLocation = 50.f;

	/** 
	 * Opacity of the second/B image (0-100).
	 * At 100%, the first/A image is rendered only on the left/top side.
	 * Below 100%, the first/A image is rendered on the entire panel with B overlaid on top of it, translucently.
	 */
	UPROPERTY(EditAnywhere, Config, Category = "AB View", DisplayName = "B Image Opacity",
		meta = (UIMin = 0, ClampMin = 0, UIMax = 100, ClampMax = 100, Units = "Percent"))
	float SecondImageOpacity = 100.f;

	UPROPERTY(EditAnywhere, Config, Category = "Media")
	EMediaViewerMediaSyncType MediaSyncType = EMediaViewerMediaSyncType::RelativeTime;
};
