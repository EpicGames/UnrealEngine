// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaImageViewerDetails.h"

#include "ImageViewer/MediaImageViewer.h"
#include "MediaViewer.h"
#include "MediaViewerDelegates.h"
#include "MediaViewerUtils.h"
#include "UObject/StructOnScope.h"
#include "Widgets/Layout/SScrollBox.h"

#define LOCTEXT_NAMESPACE "SMediaImageViewerDetails"

namespace UE::MediaViewer::Private
{

SMediaImageViewerDetails::SMediaImageViewerDetails()
{
}

void SMediaImageViewerDetails::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

void SMediaImageViewerDetails::Construct(const FArguments& InArgs, EMediaImageViewerPosition InPosition, 
	const TSharedRef<FMediaViewerDelegates>& InDelegates)
{
	Position = InPosition;
	Delegates = InDelegates;

	TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position);

	if (!ImageViewer.IsValid())
	{
		return;
	}

	TSharedRef<SScrollBox> Container = SNew(SScrollBox)
		.Orientation(Orient_Vertical);

	if (TSharedPtr<SWidget> CustomSettingsWidget = CreateCustomSettings())
	{
		Container->AddSlot()
			.AutoSize()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				CustomSettingsWidget.ToSharedRef()
			];
	}

	Container->AddSlot()
		.AutoSize()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			CreatePaintSettings()
		];

	Container->AddSlot()
		.AutoSize()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			CreatePanelSettings()
		];

	ChildSlot
	[
		Container
	];
}

TSharedRef<SWidget> SMediaImageViewerDetails::CreatePanelSettings()
{
	TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position);

	if (!ImageViewer.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	PanelDetailsView = FMediaViewerUtils::CreateStructDetailsView(
		MakeShared<FStructOnScope>(FMediaImageViewerPanelSettings::StaticStruct(), reinterpret_cast<uint8*>(&ImageViewer->GetPanelSettings())),
		LOCTEXT("PanelSettings", "Panel"),
		ImageViewer.Get()
	);

	return PanelDetailsView->GetWidget().ToSharedRef();
}

TSharedRef<SWidget> SMediaImageViewerDetails::CreatePaintSettings()
{
	TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position);

	if (!ImageViewer.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	PaintDetailsView = FMediaViewerUtils::CreateStructDetailsView(
		MakeShared<FStructOnScope>(FMediaImagePaintSettings::StaticStruct(), reinterpret_cast<uint8*>(&ImageViewer->GetPaintSettings())),
		LOCTEXT("PaintSettings", "Media (Base)"),
		ImageViewer.Get()
	);

	return PaintDetailsView->GetWidget().ToSharedRef();
}

TSharedPtr<SWidget> SMediaImageViewerDetails::CreateCustomSettings()
{
	TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position);

	if (!ImageViewer.IsValid())
	{
		return nullptr;
	}

	TSharedPtr<FStructOnScope> CustomStruct = ImageViewer->GetCustomSettingsOnScope();

	if (!CustomStruct.IsValid())
	{
		return nullptr;
	}

	CustomDetailsView = FMediaViewerUtils::CreateStructDetailsView(
		CustomStruct.ToSharedRef(),
		LOCTEXT("CustomSettings", "Media (Custom)"),
		ImageViewer.Get()
	);

	return CustomDetailsView->GetWidget().ToSharedRef();
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
