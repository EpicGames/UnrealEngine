// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FStructOnScope;
class IStructureDetailsView;

namespace UE::MediaViewer
{
	class FMediaImageViewer;
	enum class EMediaImageViewerPosition : uint8;
	struct FMediaViewerDelegates;
}

namespace UE::MediaViewer::Private
{

class SMediaImageViewerDetails : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SMediaImageViewerDetails, SCompoundWidget)

	SLATE_BEGIN_ARGS(SMediaImageViewerDetails)
		{}
	SLATE_END_ARGS()

public:
	SMediaImageViewerDetails();
	virtual ~SMediaImageViewerDetails() = default;

	void Construct(const FArguments& InArgs, EMediaImageViewerPosition InPosition, const TSharedRef<FMediaViewerDelegates>& InDelegates);

protected:
	EMediaImageViewerPosition Position;
	TSharedPtr<FMediaViewerDelegates> Delegates;

	TSharedPtr<IStructureDetailsView> PanelDetailsView;
	TSharedPtr<IStructureDetailsView> PaintDetailsView;
	TSharedPtr<IStructureDetailsView> CustomDetailsView;

	TSharedRef<SWidget> CreatePanelSettings();

	TSharedRef<SWidget> CreatePaintSettings();

	TSharedPtr<SWidget> CreateCustomSettings();
};

} // UE::MediaViewer::Private
