// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaImageViewerOverlay.h"

#include "DetailLayoutBuilder.h"
#include "EditorViewportClient.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ImageViewer/MediaImageViewer.h"
#include "ImageViewers/NullImageViewer.h"
#include "IMediaViewerModule.h"
#include "Library/MediaViewerLibrary.h"
#include "Library/MediaViewerLibraryItem.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "MediaViewer.h"
#include "MediaViewerCommands.h"
#include "MediaViewerDelegates.h"
#include "Widgets/Layout/SScissorRectBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SMediaImageViewerStatusBar.h"
#include "Widgets/SMediaViewer.h"
#include "Widgets/SMediaViewerDropTarget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SMediaImageViewerOverlay"

namespace UE::MediaViewer::Private
{

const FName IgnoreOverlayToggle = TEXT("IgnoreOverlayToggle");

SMediaImageViewerOverlay::SMediaImageViewerOverlay()
	: CommandList(MakeShared<FUICommandList>())
	, bComparisonView(false)
	, DragButtonName(NAME_None)
	, DragType(EDragType::None)
	, bOverlayEnabled(true)
{	
}

void SMediaImageViewerOverlay::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

void SMediaImageViewerOverlay::Construct(const FArguments& InArgs, EMediaImageViewerPosition InPosition, 
	const TSharedRef<FMediaViewerDelegates>& InDelegates)
{
	Position = InPosition;
	Delegates = InDelegates;
	bComparisonView = InArgs._bComparisonView;
	bScaleToFit = InArgs._bScaleToFit;

	TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position);

	if (!ImageViewer.IsValid())
	{
		return;
	}

	CachedItem = ImageViewer->CreateLibraryItem();
}

void SMediaImageViewerOverlay::PostConstruct()
{
	if (!Delegates.IsValid())
	{
		return;
	}

	BindCommands();

	ChildSlot
	[
		SNew(SScissorRectBox)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				CreateStatusBar(Delegates.ToSharedRef())
			]

			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				CreateOverlay(bComparisonView)
			]
		]
	];
}

TSharedPtr<FMediaImageViewer> SMediaImageViewerOverlay::GetImageViewer() const
{
	return Delegates->GetImageViewer.Execute(Position);
}

const TSharedRef<FUICommandList>& SMediaImageViewerOverlay::GetCommandList() const
{
	return CommandList;
}

FIntPoint SMediaImageViewerOverlay::GetImageViewerPixelCoordinates() const
{
	const FVector2D CursorLocation = GetImageViewerPixelCoordinates_Exact();

	return FIntPoint(
		FMath::RoundToInt(CursorLocation.X),
		FMath::RoundToInt(CursorLocation.Y)
	);
}

FVector2D SMediaImageViewerOverlay::GetImageViewerPixelCoordinates_Exact() const
{
	TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position);

	if (!ImageViewer.IsValid())
	{
		return FVector2D(-1, -1);
	}

	const FVector2D ViewerPosition = Delegates->GetViewerPosition.Execute();
	const FVector2D ViewerSize = Delegates->GetViewerSize.Execute();
	const FVector2D TopLeft = ImageViewer->GetPaintOffset(ViewerSize, Delegates->GetViewerPosition.Execute());

	FVector2D CursorLocation = Delegates->GetCursorLocation.Execute() - ViewerPosition;
	CursorLocation.X -= TopLeft.X;
	CursorLocation.Y -= TopLeft.Y;
	CursorLocation.X += ViewerPosition.X;
	CursorLocation.Y += ViewerPosition.Y;
	CursorLocation /= ImageViewer->GetPaintSettings().Scale;

	return CursorLocation;
}

bool SMediaImageViewerOverlay::IsCursorOverImageViewer() const
{
	TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position);

	if (!ImageViewer.IsValid())
	{
		return false;
	}

	const FIntPoint PixelCoordinates = GetImageViewerPixelCoordinates();

	if (PixelCoordinates.X < 0 || PixelCoordinates.Y < 0)
	{
		return false;
	}

	const FIntPoint& ImageSize = ImageViewer->GetInfo().Size;

	if (PixelCoordinates.X >= ImageSize.X || PixelCoordinates.Y >= ImageSize.Y)
	{
		return false;
	}

	return true;
}

FCursorReply SMediaImageViewerOverlay::OnCursorQuery(const FGeometry& InMyGeometry, const FPointerEvent& InCursorEvent) const
{
	if (DragType != EDragType::None)
	{
		return FCursorReply::Cursor(EMouseCursor::GrabHandClosed);
	}

	if (IsCursorOverImageViewer())
	{
		return FCursorReply::Cursor(EMouseCursor::Crosshairs);
	}

	return SCompoundWidget::OnCursorQuery(InMyGeometry, InCursorEvent);
}

FReply SMediaImageViewerOverlay::OnKeyDown(const FGeometry& InMyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return SCompoundWidget::OnKeyDown(InMyGeometry, InKeyEvent);
}

FReply SMediaImageViewerOverlay::OnMouseButtonDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (DragType == EDragType::None)
	{
		const FVector2D CursorLocation = Delegates->GetCursorLocation.Execute();

		if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton
			|| (InMouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton && InMouseEvent.IsAltDown()))
		{
			DragStartCursor = CursorLocation;
			DragStartOffset = GetOffset();
			DragButtonName = InMouseEvent.GetEffectingButton().GetFName();
			DragType = EDragType::Offset;
			return FReply::Handled().CaptureMouse(SharedThis(this));
		}
		else if (InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton && InMouseEvent.IsAltDown())
		{
			DragStartCursor = CursorLocation;
			LastDragScaleValue = GetScale();
			DragButtonName = InMouseEvent.GetEffectingButton().GetFName();
			DragType = EDragType::Scale;
			return FReply::Handled().CaptureMouse(SharedThis(this));
		}

		if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
		{
			const FInputEventState EventState(nullptr, InMouseEvent.GetEffectingButton(), IE_Pressed);

			if (ImageViewer->OnTrackingStarted(EventState, FIntPoint(FMath::RoundToInt(CursorLocation.X), FMath::RoundToInt(CursorLocation.Y))))
			{
				DragButtonName = InMouseEvent.GetEffectingButton().GetFName();
				DragType = EDragType::External;
				return FReply::Handled().CaptureMouse(SharedThis(this));
			}
		}
	}

	return SCompoundWidget::OnMouseButtonDown(InMyGeometry, InMouseEvent);
}

FReply SMediaImageViewerOverlay::OnMouseButtonUp(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (DragType == EDragType::None)
	{
		if (InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			FSlateApplication::Get().PushMenu(
				SharedThis(this),
				FWidgetPath(),
				CreateMenu(),
				FSlateApplication::Get().GetCursorPos(),
				FPopupTransitionEffect::ContextMenu
			);

			return FReply::Handled();
		}
	}
	else
	{
		OnDragButtonUp(InMouseEvent.GetEffectingButton().GetFName());
		return SCompoundWidget::OnMouseButtonUp(InMyGeometry, InMouseEvent).ReleaseMouseCapture();
	}

	return SCompoundWidget::OnMouseButtonUp(InMyGeometry, InMouseEvent);
}

FReply SMediaImageViewerOverlay::OnMouseWheel(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	const float ModifierScale = GetScaleModifier();

	const float ScaleMultiplier = InMouseEvent.GetWheelDelta() > 0
		? ModifierScale
		: (1.f / ModifierScale);

	if (Delegates->AreTransformsLocked.Execute())
	{
		Delegates->MultiplyScaleAroundCursorToAll.Execute(ScaleMultiplier);
	}
	else
	{
		MultiplyScaleAroundCursor(ScaleMultiplier);
	}

	return SCompoundWidget::OnMouseWheel(InMyGeometry, InMouseEvent);
}

bool SMediaImageViewerOverlay::SupportsKeyboardFocus() const
{
	return true;
}

void SMediaImageViewerOverlay::Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(InAllottedGeometry, InCurrentTime, InDeltaTime);

	if (bScaleToFit)
	{
		const FVector2D LocalSize = Delegates->GetViewerSize.Execute();

		if (FMath::IsNearlyZero(LocalSize.X) || FMath::IsNearlyZero(LocalSize.Y))
		{
			return;
		}

		TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position);

		if (ImageViewer.IsValid() && ImageViewer->GetInfo().Size.X > 2)
		{
			ScaleToFit(/* Use transform lock */ false);
			bScaleToFit = false;
		}
	}
}

void SMediaImageViewerOverlay::UpdateMouse(const TOptional<FVector2D>& InMousePosition)
{
	if (!DragButtonName.IsNone())
	{
		if (!FSlateApplication::Get().GetPressedMouseButtons().Contains(DragButtonName))
		{
			OnDragButtonUp(DragButtonName);
		}
	}

	switch (DragType)
	{
		case EDragType::Offset:
			UpdateDragPosition();
			break;

		case EDragType::Scale:
			UpdateDragScale();
			break;

		case EDragType::External:
			UpdateDragExternal();
			break;
	}

	if (InMousePosition.IsSet())
	{
		if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
		{
			ImageViewer->OnMouseMove(InMousePosition.GetValue());
		}
	}
}

void SMediaImageViewerOverlay::OnDragButtonUp(FName InKeyName)
{
	if (InKeyName != DragButtonName)
	{
		return;
	}

	switch (DragType)
	{
		case EDragType::Offset:
			UpdateDragPosition();
			break;

		case EDragType::Scale:
			UpdateDragScale();
			break;

		case EDragType::External:
			if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
			{
				const FVector2D CursorLocation = Delegates->GetCursorLocation.Execute();
				ImageViewer->OnTrackingStopped(FIntPoint(FMath::RoundToInt(CursorLocation.X), FMath::RoundToInt(CursorLocation.Y)));
			}
			break;
	}

	DragType = EDragType::None;
	DragButtonName = NAME_None;
}

void SMediaImageViewerOverlay::UpdateDragPosition()
{
	const FVector2D CursorLocation = Delegates->GetCursorLocation.Execute();

	FVector NewOffset = DragStartOffset;
	NewOffset.X += CursorLocation.X - DragStartCursor.X;
	NewOffset.Y += CursorLocation.Y - DragStartCursor.Y;

	const FVector OffsetDiff = NewOffset - GetOffset();

	if (!OffsetDiff.IsNearlyZero())
	{
		Try_AddOffset(OffsetDiff);
	}
}

void SMediaImageViewerOverlay::UpdateDragScale()
{
	static const float DragMultiplier = 0.001f;
	static const float NormalMultiplier = 1.f;
	static const float FastMultiplier = 4.f;
	static const float SlowMultiplier = 0.25f;

	const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();

	const float ModifierScale = ModifierKeys.IsLeftShiftDown()
		? FastMultiplier
		: ((ModifierKeys.IsLeftControlDown() || ModifierKeys.IsLeftCommandDown())
			? SlowMultiplier
			: NormalMultiplier);

	const FVector2D CursorLocation = Delegates->GetCursorLocation.Execute();
	const float Difference = (CursorLocation - DragStartCursor).Y;
	const float NewScale = LastDragScaleValue - Difference * ModifierScale * DragMultiplier;

	if (FMath::IsNearlyEqual(NewScale, GetScale()))
	{
		return;
	}
	
	// Since we can change speed of the scale change mid operation, reset this each update.
	DragStartCursor = CursorLocation;
	LastDragScaleValue = NewScale;

	Try_SetScale(NewScale);
}

void SMediaImageViewerOverlay::UpdateDragExternal()
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		const FVector2D CursorLocation = Delegates->GetCursorLocation.Execute();
		ImageViewer->OnMouseMove(CursorLocation);
	}
}

EVisibility SMediaImageViewerOverlay::GetDragDescriptionVisibility() const
{
	return FSlateApplication::Get().IsDragDropping()
		? EVisibility::HitTestInvisible
		: EVisibility::Collapsed;
}

void SMediaImageViewerOverlay::BindCommands()
{
	if (Delegates.IsValid() && Delegates->GetCommandList.IsBound())
	{
		if (TSharedPtr<FUICommandList> ParentCommandList = Delegates->GetCommandList.Execute())
		{
			CommandList->Append(ParentCommandList.ToSharedRef());
		}
	}

	const FMediaViewerCommands& Commands = FMediaViewerCommands::Get();

	CommandList->MapAction(Commands.MoveLeft,     FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_AddOffset, FVector(-10,   0,   0)));
	CommandList->MapAction(Commands.MoveRight,    FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_AddOffset, FVector( 10,   0,   0)));
	CommandList->MapAction(Commands.MoveUp,       FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_AddOffset, FVector(  0, -10,   0)));
	CommandList->MapAction(Commands.MoveDown,     FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_AddOffset, FVector(  0,  10,   0)));
	CommandList->MapAction(Commands.MoveForward,  FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_AddOffset, FVector(  0,   0, -10)));
	CommandList->MapAction(Commands.MoveBackward, FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_AddOffset, FVector(  0,   0,  10)));

	CommandList->MapAction(Commands.RotateMinusPitch, FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_AddRotation, FRotator(-10,   0,   0)));
	CommandList->MapAction(Commands.RotatePlusPitch,  FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_AddRotation, FRotator( 10,   0,   0)));
	CommandList->MapAction(Commands.RotateMinusYaw,   FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_AddRotation, FRotator(  0, -10,   0)));
	CommandList->MapAction(Commands.RotatePlusYaw,    FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_AddRotation, FRotator(  0,  10,   0)));
	CommandList->MapAction(Commands.RotateMinusRoll,  FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_AddRotation, FRotator(  0,   0, -10)));
	CommandList->MapAction(Commands.RotatePlusRoll,   FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_AddRotation, FRotator(  0,   0,  10)));

	CommandList->MapAction(Commands.ScaleUp, FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::ScaleUp));
	CommandList->MapAction(Commands.ScaleDown, FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::ScaleDown));

	CommandList->MapAction(
		Commands.ToggleAutoFiltering,
		FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::ToggleAutoFiltering),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSP(this, &SMediaImageViewerOverlay::IsAutoFilteringEnabled)
	);

	CommandList->MapAction(
		Commands.TogglePixelGrid,
		FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::TogglePixelGrid),
		FCanExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::CanTogglePixelGrid),
		FGetActionCheckState::CreateSP(this, &SMediaImageViewerOverlay::IsPixelGridEnabled)
	);

	CommandList->MapAction(
		Commands.ToggleMipOverride,
		FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::ToggleMipOverride),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSP(this, &SMediaImageViewerOverlay::GetMipOverrideState)
	);

	CommandList->MapAction(Commands.IncreaseMipLevel, FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::IncreaseMipLevel));
	CommandList->MapAction(Commands.DecreaseMipLevel, FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::DecreaseMipLevel));

	CommandList->MapAction(
		Commands.ScaleToFit, 
		FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::ScaleToFit, /* Use transform lock */ true)
	);

	CommandList->MapAction(Commands.Scale12,  FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_SetScale, 0.125f));
	CommandList->MapAction(Commands.Scale25,  FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_SetScale, 0.25f));
	CommandList->MapAction(Commands.Scale50,  FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_SetScale, 0.5f));
	CommandList->MapAction(Commands.Scale100, FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_SetScale, 1.0f));
	CommandList->MapAction(Commands.Scale200, FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_SetScale, 2.0f));
	CommandList->MapAction(Commands.Scale400, FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_SetScale, 4.0f));
	CommandList->MapAction(Commands.Scale800,  FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_SetScale,  8.0f));
	CommandList->MapAction(Commands.Scale1000, FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_SetScale, 10.0f));

	CommandList->MapAction(
		Commands.ResetTransform, 
		FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_ResetTransform)
	);

	CommandList->MapAction(
		Commands.ResetAllTransforms,
		FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::ResetAllTransforms),
		FCanExecuteAction(),
		FGetActionCheckState(),
		FIsActionButtonVisible::CreateSP(this, &SMediaImageViewerOverlay::CanResetAllTransforms)
	);

	CommandList->MapAction(
		Commands.CopyTransform, 
		FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::CopyTransform),
		FCanExecuteAction(),
		FGetActionCheckState(),
		FIsActionButtonVisible::CreateSP(this, &SMediaImageViewerOverlay::CanCopyTransform)
	);

	CommandList->MapAction(
		Commands.MirrorHorizonal,
		FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_Mirror, EMediaImageMirrorPlane::Horizontal),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSP(this, &SMediaImageViewerOverlay::GetMirroredState, EMediaImageMirrorPlane::Horizontal)
	);

	CommandList->MapAction(
		Commands.MirrorVertical,
		FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_Mirror, EMediaImageMirrorPlane::Vertical),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSP(this, &SMediaImageViewerOverlay::GetMirroredState, EMediaImageMirrorPlane::Vertical)
	);

	CommandList->MapAction(
		Commands.MirrorDepth,
		FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::Try_Mirror, EMediaImageMirrorPlane::Depth),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSP(this, &SMediaImageViewerOverlay::GetMirroredState, EMediaImageMirrorPlane::Depth)
	);

	CommandList->MapAction(Commands.CopyColor, FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::CopyColor));

	CommandList->MapAction(
		Commands.AddToLibrary,
		FExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::AddToLibrary),
		FCanExecuteAction::CreateSP(this, &SMediaImageViewerOverlay::CanAddToLibrary)
	);
}

TSharedRef<SWidget> SMediaImageViewerOverlay::CreateOverlay(bool bInComparisonView)
{
	TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position);

	if (!ImageViewer.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	Overlay = SNew(SOverlay);

	TSharedPtr<SWidget> ImageViewerOverlay = ImageViewer.IsValid() 
		? ImageViewer->GetOverlayWidget(Position, Delegates) 
		: nullptr;

	Overlay->AddSlot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Visibility(SMediaImageViewerOverlay::HintText_GetVisibility())
			.Text(LOCTEXT("DropTargetMessage", "Drop supported asset or library item here."))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.TextStyle(FAppStyle::Get(), "HintText")
			.Tag(IgnoreOverlayToggle)
		];

	if (bInComparisonView)
	{
		Overlay->AddSlot()
			[
				SNew(SMediaViewerDropTarget, Delegates.ToSharedRef())
				.Position(Position)
				.bComparisonView(bInComparisonView)
				.Tag(IgnoreOverlayToggle)
			];
	}
	else
	{
		Overlay->AddSlot()
			[
				SNew(SHorizontalBox)
				.Tag(IgnoreOverlayToggle)
				+ SHorizontalBox::Slot()
				.FillWidth(0.25f)
				.Padding(0.f, 0.f, 0.f, 20.f)
				[
					SNew(SMediaViewerDropTarget, Delegates.ToSharedRef())
					.Position(EMediaImageViewerPosition::First)
					.bComparisonView(bInComparisonView)
					.bForceComparisonView(true)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(0.5f)
				.Padding(10.f, 0.f, 10.f, 20.f)
				[
					SNew(SMediaViewerDropTarget, Delegates.ToSharedRef())
					.Position(EMediaImageViewerPosition::First)
					.bComparisonView(bInComparisonView)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(0.25f)
				.Padding(0.f, 0.f, 0.f, 20.f)
				[
					SNew(SMediaViewerDropTarget, Delegates.ToSharedRef())
					.Position(EMediaImageViewerPosition::Second)
					.bComparisonView(bInComparisonView)
				]
			];
	}

	if (ImageViewerOverlay.IsValid())
	{
		Overlay->AddSlot()
			[
				ImageViewerOverlay.ToSharedRef()
			];
	}

	if (bInComparisonView)
	{
		const EHorizontalAlignment HAlign = (Position == EMediaImageViewerPosition::First
			&& Delegates->GetActiveView.Execute() == EMediaImageViewerActivePosition::Both
			&& Delegates->GetABOrientation.Execute() == Orient_Horizontal)
			? HAlign_Right
			: HAlign_Left;

		Overlay->AddSlot()
			[
				SNew(SBox)
				.HAlign(HAlign)
				.VAlign(VAlign_Top)
				.Padding(5.f)
				.Visibility(EVisibility::HitTestInvisible)
				.Tag(IgnoreOverlayToggle)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ColorAndOpacity(FStyleColors::Foreground.GetSpecifiedColor())
					.ShadowColorAndOpacity(FStyleColors::Panel.GetSpecifiedColor())
					.ShadowOffset(FVector2D(1.f, 1.f))
					.Text(Position == EMediaImageViewerPosition::First ? INVTEXT("A") : INVTEXT("B"))
				]
			];
	}

	Overlay->AddSlot()
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Top)
			.Padding(5.f)
			.Visibility(EVisibility::HitTestInvisible)
			.Tag(IgnoreOverlayToggle)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ColorAndOpacity(FStyleColors::Foreground.GetSpecifiedColor())
				.ShadowColorAndOpacity(FStyleColors::Panel.GetSpecifiedColor())
				.ShadowOffset(FVector2D(1.f, 1.f))
				.Text(ImageViewer->GetInfo().DisplayName)
			]
		];

	return Overlay.ToSharedRef();
}

TSharedRef<SWidget> SMediaImageViewerOverlay::CreateStatusBar(const TSharedRef<FMediaViewerDelegates>& InDelegates)
{
	return SAssignNew(StatusBar, SMediaImageViewerStatusBar, Position, InDelegates);
}

bool SMediaImageViewerOverlay::IsOverlayEnabled() const
{
	return bOverlayEnabled;
}

void SMediaImageViewerOverlay::ToggleOverlay()
{
	bOverlayEnabled = !bOverlayEnabled;
	const EVisibility NewVisibility = bOverlayEnabled ? EVisibility::Visible : EVisibility::Collapsed;

	if (StatusBar.IsValid())
	{
		StatusBar->SetVisibility(NewVisibility);
	}

	if (Overlay.IsValid())
	{
		FChildren* Children = Overlay->GetChildren();

		for (int32 Index = 0; Index < Children->Num(); ++Index)
		{
			TSharedRef<SWidget> Child = Children->GetChildAt(Index);

			if (Child->GetTag() != IgnoreOverlayToggle)
			{
				Child->SetVisibility(NewVisibility);
			}
		}
	}
}

void SMediaImageViewerOverlay::MultiplyScaleAroundCursor(float InMultipler)
{
	const FVector2D CursorLocationBefore = GetImageViewerPixelCoordinates_Exact();

	SetScale(GetScale() * InMultipler);

	const FVector2D CursorLocationAfter = GetImageViewerPixelCoordinates_Exact();

	if (!(CursorLocationBefore - CursorLocationAfter).IsNearlyZero())
	{
		FVector Offset;
		Offset.X = CursorLocationAfter.X - CursorLocationBefore.X;
		Offset.Y = CursorLocationAfter.Y - CursorLocationBefore.Y;
		Offset.Z = 0;

		Offset *= GetScale();

		AddOffset(Offset);
	}
}

void SMediaImageViewerOverlay::ResetTransform()
{
	SetOffset(FVector::ZeroVector);
	SetRotation(FRotator::ZeroRotator);
	ScaleToFit(/* Use transform lock */ false);
}

bool SMediaImageViewerOverlay::CanResetAllTransforms()
{
	return Delegates->GetActiveView.Execute() == EMediaImageViewerActivePosition::Both;
}

void SMediaImageViewerOverlay::ResetAllTransforms()
{
	Delegates->ResetTransformToAll.Execute();
}

FVector SMediaImageViewerOverlay::GetOffset() const
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		return ImageViewer->GetPaintSettings().Offset;
	}

	return FVector::ZeroVector;
}

void SMediaImageViewerOverlay::Try_AddOffset(FVector InOffset)
{
	if (Delegates->AreTransformsLocked.Execute())
	{
		Delegates->AddOffsetToAll.Execute(InOffset);
	}
	else
	{
		AddOffset(InOffset);
	}
}

void SMediaImageViewerOverlay::AddOffset(const FVector& InOffset)
{
	SetOffset(GetOffset() + InOffset);
}

void SMediaImageViewerOverlay::SetOffset(const FVector& InOffset)
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		ImageViewer->GetPaintSettings().Offset = InOffset;
	}
}

FRotator SMediaImageViewerOverlay::GetRotation() const
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		return ImageViewer->GetPaintSettings().Rotation;
	}

	return FRotator::ZeroRotator;
}

void SMediaImageViewerOverlay::Try_AddRotation(FRotator InRotation)
{
	if (Delegates->AreTransformsLocked.Execute())
	{
		Delegates->AddRotationToAll.Execute(InRotation);
	}
	else
	{
		AddRotation(InRotation);
	}
}

void SMediaImageViewerOverlay::AddRotation(const FRotator& InRotation)
{
	SetRotation(GetRotation() + InRotation);
}

void SMediaImageViewerOverlay::SetRotation(const FRotator& InRotation)
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		ImageViewer->GetPaintSettings().Rotation = InRotation;
	}
}

float SMediaImageViewerOverlay::GetScale() const
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		return ImageViewer->GetPaintSettings().Scale;
	}
	
	return 1.f;
}

void SMediaImageViewerOverlay::Try_SetScale(float InScale)
{
	if (Delegates->AreTransformsLocked.Execute())
	{
		Delegates->MultiplyScaleToAll.Execute(InScale / GetScale());
	}
	else
	{
		SetScale(InScale);
	}
}

void SMediaImageViewerOverlay::SetScale(float InScale)
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		ImageViewer->GetPaintSettings().Scale = FMath::Clamp(InScale, SMediaViewer::MinScale, SMediaViewer::MaxScale);
	}
}

float SMediaImageViewerOverlay::GetScaleModifier()
{
	static const float NormalMultiplier = FMath::Pow(2.0, 0.125);
	static const float FastMultiplier = FMath::Pow(2.0, 0.5);
	static const float SlowMultiplier = FMath::Pow(2.0, 0.03125);

	const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();

	return ModifierKeys.IsLeftShiftDown()
		? FastMultiplier
		: ((ModifierKeys.IsLeftControlDown() || ModifierKeys.IsLeftCommandDown())
			? SlowMultiplier
			: NormalMultiplier);
}

void SMediaImageViewerOverlay::ScaleUp()
{
	Try_MultipyScale(GetScaleModifier());
}

void SMediaImageViewerOverlay::ScaleDown()
{
	Try_MultipyScale(1.f / GetScaleModifier());
}

void SMediaImageViewerOverlay::Try_MultipyScale(float InMultiple)
{
	if (Delegates->AreTransformsLocked.Execute())
	{
		Delegates->MultiplyScaleToAll.Execute(InMultiple);
	}
	else
	{
		SetScale(GetScale() * InMultiple);
	}
}

void SMediaImageViewerOverlay::MultiplyScale(float InMultiple)
{
	SetScale(GetScale() * InMultiple);
}

void SMediaImageViewerOverlay::ScaleToFit(bool bInUseTransformLock)
{
	TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position);

	if (!ImageViewer.IsValid())
	{
		return;
	}

	SetOffset(FVector::ZeroVector);

	const FVector2D LocalSize = Delegates->GetViewerSize.Execute();

	if (FMath::IsNearlyZero(LocalSize.X) || FMath::IsNearlyZero(LocalSize.Y))
	{
		return;
	}

	const FIntPoint& ImageSize = ImageViewer->GetInfo().Size;

	const float ScaleX = LocalSize.X / static_cast<float>(ImageSize.X);
	const float ScaleY = LocalSize.Y / static_cast<float>(ImageSize.Y);

	if (ScaleX < ScaleY)
	{
		if (bInUseTransformLock)
		{
			Try_SetScale(ScaleX);
		}
		else
		{
			SetScale(ScaleX);
		}
	}
	else
	{
		if (bInUseTransformLock)
		{
			Try_SetScale(ScaleY);
		}
		else
		{
			SetScale(ScaleY);
		}
	}
}

ECheckBoxState SMediaImageViewerOverlay::IsAutoFilteringEnabled() const
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		return ImageViewer->GetPaintSettings().bAutoFilteringEnabled
			? ECheckBoxState::Checked
			: ECheckBoxState::Unchecked;
	}

	return ECheckBoxState::Undetermined;
}

void SMediaImageViewerOverlay::ToggleAutoFiltering()
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		ImageViewer->GetPaintSettings().bAutoFilteringEnabled = !ImageViewer->GetPaintSettings().bAutoFilteringEnabled;
	}
}

bool SMediaImageViewerOverlay::CanTogglePixelGrid() const
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		return ImageViewer->GetPaintSettings().bAutoFilteringEnabled;
	}

	return false;
}

ECheckBoxState SMediaImageViewerOverlay::IsPixelGridEnabled() const
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		return ImageViewer->GetPaintSettings().bPixelGridEnabled
			? ECheckBoxState::Checked
			: ECheckBoxState::Unchecked;
	}

	return ECheckBoxState::Undetermined;
}

void SMediaImageViewerOverlay::TogglePixelGrid()
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		ImageViewer->GetPaintSettings().bPixelGridEnabled = !ImageViewer->GetPaintSettings().bPixelGridEnabled;
	}
}

ECheckBoxState SMediaImageViewerOverlay::GetMipOverrideState() const
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		return ImageViewer->GetPaintSettings().bEnableMipOverride
			? ECheckBoxState::Checked
			: ECheckBoxState::Unchecked;
	}

	return ECheckBoxState::Undetermined;
}

void SMediaImageViewerOverlay::ToggleMipOverride()
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		ImageViewer->GetPaintSettings().bEnableMipOverride = !ImageViewer->GetPaintSettings().bEnableMipOverride;
	}
}

void SMediaImageViewerOverlay::IncreaseMipLevel()
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		if (ImageViewer->GetPaintSettings().MipOverride < 10)
		{
			ImageViewer->GetPaintSettings().MipOverride++;
		}
	}
}

void SMediaImageViewerOverlay::DecreaseMipLevel()
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		if (ImageViewer->GetPaintSettings().MipOverride > 0)
		{
			ImageViewer->GetPaintSettings().MipOverride--;
		}
	}
}

void SMediaImageViewerOverlay::Try_ResetTransform()
{
	if (Delegates->AreTransformsLocked.Execute())
	{
		Delegates->ResetTransformToAll.Execute();
	}
	else
	{
		ResetTransform();
	}
}

void SMediaImageViewerOverlay::Try_SetTransform(FVector InOffset, FRotator InRotation, float InScale)
{
	if (Delegates->AreTransformsLocked.Execute())
	{
		Delegates->SetTransformToAll.Execute(InOffset, InRotation, InScale);
	}
	else
	{
		SetOffset(InOffset);
		SetRotation(InRotation);
		SetScale(InScale);
	}
}

bool SMediaImageViewerOverlay::CanCopyTransform()
{
	return Delegates->GetActiveView.Execute() == EMediaImageViewerActivePosition::Both;
}

void SMediaImageViewerOverlay::CopyTransform()
{
	Delegates->SetTransformToAll.Execute(GetOffset(), GetRotation(), GetScale());
}

void SMediaImageViewerOverlay::Try_Mirror(EMediaImageMirrorPlane InPlane)
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		if (Delegates->AreTransformsLocked.Execute())
		{
			Delegates->MirrorAll.Execute(InPlane, !ImageViewer->IsMirrored(InPlane));
		}
		else
		{
			Mirror(InPlane);
		}
	}
}

ECheckBoxState SMediaImageViewerOverlay::GetMirroredState(EMediaImageMirrorPlane InPlane) const
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		return ImageViewer->IsMirrored(InPlane)
			? ECheckBoxState::Checked
			: ECheckBoxState::Unchecked;
	}

	return ECheckBoxState::Undetermined;
}

void SMediaImageViewerOverlay::Mirror(EMediaImageMirrorPlane InPlane)
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		ImageViewer->SetMirrored(InPlane, !ImageViewer->IsMirrored(InPlane));
	}
}

uint8 SMediaImageViewerOverlay::GetMipLevel() const
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		return ImageViewer->GetPaintSettings().GetMipLevel();
	}

	return 0;
}

void SMediaImageViewerOverlay::AdjustMipLevel(int8 InAdjustment)
{
	const uint8 CurrentMipLevel = GetMipLevel();

	if (-InAdjustment > CurrentMipLevel)
	{
		SetMipLevel(0);
	}
	else
	{
		SetMipLevel(CurrentMipLevel + InAdjustment);
	}
}

void SMediaImageViewerOverlay::SetMipLevel(uint8 InMipLevel)
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		if (InMipLevel < ImageViewer->GetInfo().MipCount)
		{
			ImageViewer->GetPaintSettings().MipOverride = InMipLevel;
		}
	}
}

void SMediaImageViewerOverlay::CopyColor()
{
	TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position);

	if (!ImageViewer.IsValid() || !ImageViewer->IsValid())
	{
		return;
	}

	const FIntPoint PixelCoordinates = Delegates->GetPixelCoordinates.Execute(Position);

	if (PixelCoordinates.X < 0 || PixelCoordinates.Y < 0)
	{
		return;
	}

	const FIntPoint& ImageSize = ImageViewer->GetInfo().Size;

	if (PixelCoordinates.X >= ImageSize.X || PixelCoordinates.Y >= ImageSize.Y)
	{
		return;
	}

	const TOptional<TVariant<FColor, FLinearColor>> PixelColor = ImageViewer->GetPixelColor(PixelCoordinates, ImageViewer->GetPaintSettings().GetMipLevel());

	if (!PixelColor.IsSet())
	{
		return;
	}

	const TVariant<FColor, FLinearColor>& PixelColorValue = PixelColor.GetValue();

	if (const FColor* Color = PixelColorValue.TryGet<FColor>())
	{
		FPlatformApplicationMisc::ClipboardCopy(*Color->ToString());
	}

	if (const FLinearColor* ColorLinear = PixelColorValue.TryGet<FLinearColor>())
	{
		FPlatformApplicationMisc::ClipboardCopy(*ColorLinear->ToString());
	}
}

bool SMediaImageViewerOverlay::CanAddToLibrary() const
{
	if (!CachedItem.IsValid())
	{
		return false;
	}

	return !Delegates->GetLibrary.Execute()->GetItemGroup(CachedItem->GetId()).IsValid();
}

void SMediaImageViewerOverlay::AddToLibrary()
{
	if (!CachedItem.IsValid())
	{
		return;
	}

	TSharedRef<IMediaViewerLibrary> Library = Delegates->GetLibrary.Execute();

	Library->AddItemToGroup(CachedItem.ToSharedRef());
}

TSharedRef<SWidget> SMediaImageViewerOverlay::CreateMenu()
{
	FMediaViewerCommands& Commands = FMediaViewerCommands::Get();

	FMenuBuilder MenuBuilder(true, CommandList, nullptr, false, &FAppStyle::Get(), false);

	if (GetImageViewer() != FNullImageViewer::GetNullImageViewer())
	{
		MenuBuilder.BeginSection(TEXT("Image"), LOCTEXT("Image", "Image"));
		{
			MenuBuilder.AddMenuEntry(Commands.ResetTransform);
			MenuBuilder.AddMenuEntry(Commands.CopyTransform);
			MenuBuilder.AddMenuEntry(Commands.AddToLibrary);
			MenuBuilder.AddSeparator();
			MenuBuilder.AddMenuEntry(Commands.ToggleAutoFiltering);
			MenuBuilder.AddMenuEntry(Commands.TogglePixelGrid);
			MenuBuilder.AddSeparator();
			MenuBuilder.AddMenuEntry(Commands.ToggleMipOverride);
			MenuBuilder.AddMenuEntry(Commands.IncreaseMipLevel);
			MenuBuilder.AddMenuEntry(Commands.DecreaseMipLevel);
			MenuBuilder.AddSeparator();
			MenuBuilder.AddMenuEntry(Commands.MirrorHorizonal);
			MenuBuilder.AddMenuEntry(Commands.MirrorVertical);
			// This is disabled until 3D image viewers are added.
			//MenuBuilder.AddMenuEntry(Commands.MirrorDepth);
		}
		MenuBuilder.EndSection();
	}

	MenuBuilder.BeginSection(TEXT("Media Viewer"), LOCTEXT("MediaViewer", "Media Viewer"));
	{
		MenuBuilder.AddMenuEntry(Commands.ToggleOverlay);
		MenuBuilder.AddMenuEntry(Commands.ToggleLockedTransform);
		MenuBuilder.AddMenuEntry(Commands.ResetAllTransforms);
		MenuBuilder.AddMenuEntry(Commands.SwapAB);
	}
	MenuBuilder.EndSection();

	FNewMenuDelegate LoadBookmarksMenu = FNewMenuDelegate::CreateLambda(
		[](FMenuBuilder& InMenuBuilder)
		{
			FMediaViewerCommands& Commands = FMediaViewerCommands::Get();

			InMenuBuilder.BeginSection(TEXT("OpenBookmarks"), LOCTEXT("Bookmarks", "Bookmarks"));
			{
				InMenuBuilder.AddMenuEntry(Commands.OpenBookmark1);
				InMenuBuilder.AddMenuEntry(Commands.OpenBookmark2);
				InMenuBuilder.AddMenuEntry(Commands.OpenBookmark3);
				InMenuBuilder.AddMenuEntry(Commands.OpenBookmark4);
				InMenuBuilder.AddMenuEntry(Commands.OpenBookmark5);
				InMenuBuilder.AddMenuEntry(Commands.OpenBookmark6);
				InMenuBuilder.AddMenuEntry(Commands.OpenBookmark7);
				InMenuBuilder.AddMenuEntry(Commands.OpenBookmark8);
				InMenuBuilder.AddMenuEntry(Commands.OpenBookmark9);
				InMenuBuilder.AddMenuEntry(Commands.OpenBookmark10);
			}
			InMenuBuilder.EndSection();
		}
	);

	FNewMenuDelegate SaveBookmarksMenu = FNewMenuDelegate::CreateLambda(
		[](FMenuBuilder& InMenuBuilder)
		{
			FMediaViewerCommands& Commands = FMediaViewerCommands::Get();

			InMenuBuilder.BeginSection(TEXT("SaveBookmarks"), LOCTEXT("Bookmarks", "Bookmarks"));
			{
				InMenuBuilder.AddMenuEntry(Commands.SaveBookmark1);
				InMenuBuilder.AddMenuEntry(Commands.SaveBookmark2);
				InMenuBuilder.AddMenuEntry(Commands.SaveBookmark3);
				InMenuBuilder.AddMenuEntry(Commands.SaveBookmark4);
				InMenuBuilder.AddMenuEntry(Commands.SaveBookmark5);
				InMenuBuilder.AddMenuEntry(Commands.SaveBookmark6);
				InMenuBuilder.AddMenuEntry(Commands.SaveBookmark7);
				InMenuBuilder.AddMenuEntry(Commands.SaveBookmark8);
				InMenuBuilder.AddMenuEntry(Commands.SaveBookmark9);
				InMenuBuilder.AddMenuEntry(Commands.SaveBookmark10);
			}
			InMenuBuilder.EndSection();
		}
	);

	MenuBuilder.BeginSection(TEXT("Bookmarks"), LOCTEXT("Bookmarks", "Bookmarks"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("OpenBookmarkMenu", "Open Bookmark"),
			LOCTEXT("OpenBookmarkMenuTooltip", "Load a saved bookmark."),
			LoadBookmarksMenu
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("SaveBookmarkMenu", "Save Bookmark"),
			LOCTEXT("SaveBookmarkMenuTooltip", "Save a bookmark of the current state."),
			SaveBookmarksMenu
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

EVisibility SMediaImageViewerOverlay::HintText_GetVisibility() const
{
	if (FSlateApplication::Get().IsDragDropping())
	{
		return EVisibility::Collapsed;
	}

	TSharedPtr<FMediaImageViewer> ImageViewer = GetImageViewer();
	
	return (ImageViewer.IsValid() && ImageViewer->GetInfo().Id != FNullImageViewer::GetNullImageViewer()->GetInfo().Id)
		? EVisibility::Collapsed
		: EVisibility::Visible;
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
