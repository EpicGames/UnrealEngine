// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FUICommandList;
class SOverlay;
class SVerticalBox;
enum class ECheckBoxState : uint8;
enum class EMediaImageMirrorPlane : uint8;
struct FMediaViewerLibraryItem;

namespace UE::MediaViewer
{
	class FMediaImageViewer;
	enum class EMediaImageViewerPosition : uint8;
	struct FMediaViewerDelegates;
}

namespace UE::MediaViewer::Private
{

class SMediaImageViewerStatusBar;
class SMediaViewerLibrary;
class SMediaViewerLibraryPrivate;

/** Display and control for a viewer. Status bar, mouse and keyboard input, etc. */
class SMediaImageViewerOverlay : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SMediaImageViewerOverlay, SCompoundWidget)

	SLATE_BEGIN_ARGS(SMediaImageViewerOverlay)
		: _bComparisonView(true)
		, _bScaleToFit(false)
		{}
		SLATE_ARGUMENT(bool, bComparisonView)
		SLATE_ARGUMENT(bool, bScaleToFit)
	SLATE_END_ARGS()

public:
	SMediaImageViewerOverlay();

	virtual ~SMediaImageViewerOverlay() override = default;

	void Construct(const FArguments& InArgs, EMediaImageViewerPosition InPosition, const TSharedRef<FMediaViewerDelegates>& InDelegates);

	void PostConstruct();

	TSharedPtr<FMediaImageViewer> GetImageViewer() const;

	const TSharedRef<FUICommandList>& GetCommandList() const;

	/** Gets the pixel coordinate of mouse on the image. May be outside of the image. */
	FIntPoint GetImageViewerPixelCoordinates() const;

	/** Gets the pixel coordinate of mouse on the image. May be outside of the image. Subpixel accuracy */
	FVector2D GetImageViewerPixelCoordinates_Exact() const;

	bool IsCursorOverImageViewer() const;

	/** Triggers a mouse update. */
	void UpdateMouse(const TOptional<FVector2D>& InMousePosition);

	bool IsOverlayEnabled() const;
	void ToggleOverlay();

	/** Zooms the view in or out and maintains the cursor in the same relative position. */
	void MultiplyScaleAroundCursor(float InMultipler);
	
	/** Scales the image to fit the viewer. */
	void ScaleToFit(bool bInUseTransformLock);

	/** Either mirror this viewer or all viewers if transforms are locked. */
	void Try_Mirror(EMediaImageMirrorPlane InPlane);

	//~ Begin SWidget
	virtual FCursorReply OnCursorQuery(const FGeometry& InMyGeometry, const FPointerEvent& InCursorEvent) const override;
	virtual FReply OnKeyDown(const FGeometry& InMyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnMouseButtonDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnMouseWheel(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual bool SupportsKeyboardFocus() const override;
	virtual void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget

protected:
	enum class EDragType : uint8
	{
		/** No drag operatio is active. */
		None,
		/** Whether the widget is performing an internal offset drag operation. */
		Offset,
		/** Whether the widget is performing an internal scale drag operation. */
		Scale,
		/** Whether an image viewer is performing its own drag operation. */
		External
	};

	EMediaImageViewerPosition Position;
	TSharedPtr<FMediaViewerDelegates> Delegates;
	TSharedPtr<FMediaViewerLibraryItem> CachedItem;
	TSharedRef<FUICommandList> CommandList;
	bool bComparisonView;
	/** Name of the button which is currently performing a drag operation. */
	FName DragButtonName;
	EDragType DragType;
	FVector2D DragStartCursor;
	FVector DragStartOffset;
	float LastDragScaleValue;
	/** For moving the image based on cursor location. */
	FIntPoint LastCursorPosition;
	TSharedPtr<SMediaImageViewerStatusBar> StatusBar;
	TSharedPtr<SOverlay> Overlay;
	/** Toggles the visibility of the overlay. */
	bool bOverlayEnabled;
	/** Whether, on first valid paint, it should scale to fit. */
	bool bScaleToFit;

	void BindCommands();

	TSharedRef<SWidget> CreateOverlay(bool bInComparisonView);

	TSharedRef<SWidget> CreateStatusBar(const TSharedRef<FMediaViewerDelegates>& InDelegates);

	void OnDragButtonUp(FName InKeyName);
	void UpdateDragPosition();
	void UpdateDragScale();
	void UpdateDragExternal();

	EVisibility GetDragDescriptionVisibility() const;

	/** Interact with the image viewer's offset. */
	FVector GetOffset() const;
	void Try_AddOffset(FVector InOffset); // Either local or 'send to all' if transform is locked
	void AddOffset(const FVector& InOffset);
	void SetOffset(const FVector& InOffset);

	/** Interact with the image viewer's rotation. */
	FRotator GetRotation() const;
	void Try_AddRotation(FRotator InRotation); // Either local or 'send to all' if transform is locked
	void AddRotation(const FRotator& InRotation);
	void SetRotation(const FRotator& InRotation);

	/** Interact with the image viewer's scale. */
	float GetScale() const;
	void Try_SetScale(float InScale); // Either local or 'send to all' if transform is locked
	void SetScale(float InScale);
	float GetScaleModifier();
	void ScaleUp();
	void ScaleDown();
	void Try_MultipyScale(float InMultiple);
	void MultiplyScale(float InMultiple);

	ECheckBoxState IsAutoFilteringEnabled() const;
	void ToggleAutoFiltering();

	bool CanTogglePixelGrid() const;
	ECheckBoxState IsPixelGridEnabled() const;
	void TogglePixelGrid();

	ECheckBoxState GetMipOverrideState() const;
	void ToggleMipOverride();
	void IncreaseMipLevel();
	void DecreaseMipLevel();

	/** Interact with the image viewer's entire transform. */
	void ResetTransform();
	bool CanResetAllTransforms();
	void ResetAllTransforms();
	void Try_ResetTransform(); // Either local or 'send to all' if transform is locked
	void Try_SetTransform(FVector InOffset, FRotator InRotation, float InScale); // Either local or 'send to all' if transform is locked
	bool CanCopyTransform();
	void CopyTransform();

	/** Interact with the image viewer's mirror settings. */
	ECheckBoxState GetMirroredState(EMediaImageMirrorPlane InPlane) const;
	void Mirror(EMediaImageMirrorPlane InPlane);

	/** Interact with the image viewer's mip level. */
	uint8 GetMipLevel() const;
	void AdjustMipLevel(int8 InAdjustment);
	void SetMipLevel(uint8 InMipLevel);

	/** Copies the color under the cursor, if possible. */
	void CopyColor();

	/** For adding the active image viewer to the library, if it can. */
	bool CanAddToLibrary() const;
	void AddToLibrary();

	TSharedRef<SWidget> CreateMenu();

	EVisibility HintText_GetVisibility() const;
};

} // UE::MediaViewer::Private
