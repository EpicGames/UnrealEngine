// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaImageViewerStatusBar.h"

#include "DetailLayoutBuilder.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ImageViewer/MediaImageViewer.h"
#include "Internationalization/Text.h"
#include "MediaViewerStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/MediaImageStatusBarExtender.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SMediaImageViewerStatusBar"

namespace UE::MediaViewer::Private
{

SMediaImageViewerStatusBar::SMediaImageViewerStatusBar()
{
}

void SMediaImageViewerStatusBar::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

void SMediaImageViewerStatusBar::Construct(const FArguments& InArgs, EMediaImageViewerPosition InPosition, 
	const TSharedRef<FMediaViewerDelegates>& InDelegates)
{
	Position = InPosition;
	Delegates = InDelegates;
	check(Delegates->GetPixelCoordinates.IsBound());

	ChildSlot
	[
		BuildStatusBar()
	];
}

TSharedRef<SWidget> SMediaImageViewerStatusBar::BuildStatusBar()
{
	TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position);

	if (!ImageViewer.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	FMediaImageStatusBarExtender StatusBarExtender;
	ImageViewer->ExtendStatusBar(StatusBarExtender);

	const FMargin SlotPadding(6.0f, 2.0f);
	TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

	auto ApplyHook = [this, &StatusBarExtender , &HorizontalBox](const FName& ExtensionHook, EExtensionHook::Position HookPosition)
	{
		if (ExtensionHook != NAME_None)
		{
			StatusBarExtender.Apply(ExtensionHook, HookPosition, HorizontalBox);
		}
	};

	using namespace UE::MediaViewer;

	ApplyHook(StatusBarSections::StatusBarLeft, EExtensionHook::Before);
	{
		HorizontalBox->AddSlot()
			.Padding(SlotPadding)
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SMediaImageViewerStatusBar::GetResolutionLabel)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ColorAndOpacity(FStyleColors::Foreground.GetSpecifiedColor())
				.ShadowColorAndOpacity(FStyleColors::Panel.GetSpecifiedColor())
				.ShadowOffset(FVector2D(1.f, 1.f))
			];
	}
	ApplyHook(StatusBarSections::StatusBarLeft, EExtensionHook::After);

	ApplyHook(StatusBarSections::StatusBarCenter, EExtensionHook::Before);
	{
		HorizontalBox->AddSlot()
			.HAlign(HAlign_Center)
			[
				SNullWidget::NullWidget

				// This is deliberately left empty.
			];
	}
	ApplyHook(StatusBarSections::StatusBarCenter, EExtensionHook::After);

	ApplyHook(StatusBarSections::StatusBarRight, EExtensionHook::Before);
	{
		HorizontalBox->AddSlot()
			.Padding(SlotPadding)
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SRichTextBlock)
				.Text(this, &SMediaImageViewerStatusBar::GetColorPickerLabel)
				.DecoratorStyleSet(&FMediaViewerStyle::Get())
			];

		HorizontalBox->AddSlot()
			.Padding(SlotPadding)
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SColorBlock)
				.Size(FVector2D(12))
				.Color(this, &SMediaImageViewerStatusBar::GetColorBlockColor)
				.UseSRGB(false)
			];
	}
	ApplyHook(StatusBarSections::StatusBarRight, EExtensionHook::After);

	return HorizontalBox;
}

FText SMediaImageViewerStatusBar::GetResolutionLabel() const
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position))
	{
		return FText::Format(
			LOCTEXT("Size", "{0} x {1}"),
			FText::AsNumber(ImageViewer->GetInfo().Size.X),
			FText::AsNumber(ImageViewer->GetInfo().Size.Y)
		);
	}

	return INVTEXT("-");
}

FText SMediaImageViewerStatusBar::GetColorPickerLabel() const
{
	TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position);

	if (!ImageViewer.IsValid() || !ImageViewer->IsValid())
	{
		return INVTEXT("-");
	}

	const FIntPoint PixelCoordinates = Delegates->GetPixelCoordinates.Execute(Position);

	if (PixelCoordinates.X < 0 || PixelCoordinates.Y < 0)
	{
		return INVTEXT("-");
	}
	
	const FIntPoint& ImageSize = ImageViewer->GetInfo().Size;

	if (PixelCoordinates.X >= ImageSize.X || PixelCoordinates.Y >= ImageSize.Y)
	{
		return INVTEXT("-");
	}

	const TOptional<TVariant<FColor, FLinearColor>> PixelColor = ImageViewer->GetPixelColor(PixelCoordinates, ImageViewer->GetPaintSettings().GetMipLevel());

	if (!PixelColor.IsSet())
	{
		return FText::Format(
			LOCTEXT("CoordinatesWithoutColor", "<RichTextBlock.Normal>[{0}, {1}]</>"),
			FText::AsNumber(PixelCoordinates.X + 1),
			FText::AsNumber(PixelCoordinates.Y + 1)
		);
	}

	auto CoordsAndColorText = [&PixelCoordinates](const auto* Color, const FNumberFormattingOptions& Formatting)
	{
		return FText::Format(
			LOCTEXT("CoordinatesWithColor", "<RichTextBlock.Normal>[{0}, {1}]</> <RichTextBlock.Red>{2}</> <RichTextBlock.Green>{3}</> <RichTextBlock.Blue>{4}</> <RichTextBlock.Normal>{5}</>"),
			FText::AsNumber(PixelCoordinates.X + 1),
			FText::AsNumber(PixelCoordinates.Y + 1),
			FText::AsNumber(Color->R, &Formatting),
			FText::AsNumber(Color->G, &Formatting),
			FText::AsNumber(Color->B, &Formatting), 
			FText::AsNumber(Color->A, &Formatting)
		);
	};

	const TVariant<FColor, FLinearColor>& PixelColorValue = PixelColor.GetValue();

	if (const FColor* Color = PixelColorValue.TryGet<FColor>())
	{
		FNumberFormattingOptions FormattingByte;
		FormattingByte.SetMinimumIntegralDigits(3);
		
		return CoordsAndColorText(Color, FormattingByte);
	}
	
	if (const FLinearColor* ColorLinear = PixelColorValue.TryGet<FLinearColor>())
	{
		FNumberFormattingOptions FormattingFloat;
		FormattingFloat.SetMinimumFractionalDigits(3);
		FormattingFloat.SetMaximumFractionalDigits(3);

		return CoordsAndColorText(ColorLinear, FormattingFloat);
	}

	// Should never happen!
	return FText::GetEmpty();
}

FLinearColor SMediaImageViewerStatusBar::GetColorBlockColor() const
{
	TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(Position);

	if (!ImageViewer.IsValid() || !ImageViewer->IsValid())
	{
		return FLinearColor::Black;
	}

	const FIntPoint PixelCoordinates = Delegates->GetPixelCoordinates.Execute(Position);

	if (PixelCoordinates.X < 0 || PixelCoordinates.Y < 0)
	{
		return FLinearColor::Black;
	}

	const FIntPoint& ImageSize = ImageViewer->GetInfo().Size;

	if (PixelCoordinates.X >= ImageSize.X || PixelCoordinates.Y >= ImageSize.Y)
	{
		return FLinearColor::Black;
	}

	const TOptional<TVariant<FColor, FLinearColor>> PixelColor = ImageViewer->GetPixelColor(PixelCoordinates, ImageViewer->GetPaintSettings().GetMipLevel());

	if (!PixelColor.IsSet())
	{
		return FLinearColor::Black;
	}

	const TVariant<FColor, FLinearColor>& PixelColorValue = PixelColor.GetValue();

	if (const FLinearColor* ColorLinear = PixelColorValue.TryGet<FLinearColor>())
	{
		return *ColorLinear;
	}

	if (const FColor* Color = PixelColorValue.TryGet<FColor>())
	{
		return Color->ReinterpretAsLinear();
	}

	return FLinearColor::Black;
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
