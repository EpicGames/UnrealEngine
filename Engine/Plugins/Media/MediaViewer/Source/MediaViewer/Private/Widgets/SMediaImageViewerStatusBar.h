// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "MediaViewerDelegates.h"

class FText;
class FUICommandList;

namespace UE::MediaViewer
{
	class FMediaImageViewer;
}

namespace UE::MediaViewer::Private
{

class SMediaImageViewerStatusBar : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SMediaImageViewerStatusBar, SCompoundWidget)

	SLATE_BEGIN_ARGS(SMediaImageViewerStatusBar)
		{}
	SLATE_END_ARGS()

public:
	SMediaImageViewerStatusBar();

	virtual ~SMediaImageViewerStatusBar() override = default;

	void Construct(const FArguments& InArgs, EMediaImageViewerPosition InPosition, const TSharedRef<FMediaViewerDelegates>& InDelegates);

protected:
	EMediaImageViewerPosition Position;
	TSharedPtr<FMediaViewerDelegates> Delegates;

	TSharedRef<SWidget> BuildStatusBar();

	FText GetResolutionLabel() const;

	FText GetColorPickerLabel() const;

	FLinearColor GetColorBlockColor() const;
};

} // UE::MediaViewer::Private
