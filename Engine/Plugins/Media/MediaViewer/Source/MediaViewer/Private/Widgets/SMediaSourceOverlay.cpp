// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaSourceOverlay.h"

#include "DetailLayoutBuilder.h"
#include "ImageViewers/MediaSourceImageViewer.h"
#include "IMediaStreamPlayer.h"
#include "MediaPlayer.h"
#include "MediaStream.h"
#include "MediaViewerCommands.h"
#include "MediaViewerDelegates.h"
#include "MediaViewerStyle.h"
#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"
#include "Misc/Timespan.h"
#include "Modules/ModuleManager.h"
#include "Styling/SlateIconFinder.h"
#include "Styling/StyleColors.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SMediaViewer.h"
#include "Widgets/SMediaViewerTab.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SMediaSourceOverlay"

namespace UE::MediaViewer::Private
{

constexpr double FadeTime = 2.0;

namespace MediaSourceOverlay::Events
{
	const FLazyName Rewind      = "MediaSource_Rewind";
	const FLazyName Reverse     = "MediaSource_Reverse";
	const FLazyName StepBack    = "MediaSource_StepBack";
	const FLazyName Play        = "MediaSource_Play";
	const FLazyName Pause       = "MediaSource_Pause";
	const FLazyName StepForward = "MediaSource_StepForward";
	const FLazyName Forward     = "MediaSource_Forward";
	const FLazyName Scrub       = "MediaSource_Scrub";

	struct FMediaImageViewerScrubEventParams : FMediaImageViewerEventParams
	{
		IMediaPlayerSlider::EScrubEventType ScrubEvent;
		TVariant<FTimespan, float> ScrubData = {};
	};
}

namespace MediaSourceOverlay
{
	FText GetKeybindText(const TSharedPtr<FUICommandInfo>& InCommand)
	{
		if (InCommand.IsValid())
		{
			const TSharedRef<const FInputChord>& Chord = InCommand->GetFirstValidChord();

			if (Chord->IsValidChord())
			{
				return Chord->GetInputText(/* Long display name */ false);
			}
		}

		return FText::GetEmpty();
	}

	FText AddKeybind(const FText& InToolTip, const TSharedPtr<FUICommandInfo>& InCommand)
	{
		const FText KeybindText = GetKeybindText(InCommand);

		if (!KeybindText.IsEmpty())
		{
			return FText::Format(LOCTEXT("AddKeybind", "{0}\nKeybind: {1}"), InToolTip, KeybindText);
		}

		return InToolTip;
	}
}

void SMediaSourceOverlay::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

void SMediaSourceOverlay::Construct(const FArguments& InArgs, const TSharedRef<FMediaSourceImageViewer>& InImageViewer,
	EMediaImageViewerPosition InPosition, const TSharedPtr<FMediaViewerDelegates>& InDelegates)
{
	ImageViewerWeak = InImageViewer;
	Position = InPosition;
	RateWhenScrubEventReceived = {};

	if (InDelegates.IsValid())
	{
		Delegates = InDelegates;
		Delegates->ImageViewerEvent.AddSP(this, &SMediaSourceOverlay::ReceiveEvent);
	}

	BindCommands();

	TrySetFrameRate();

	ChildSlot
	[
		SAssignNew(Container, SBox)
		[
			SNew(SBorder)
			.Padding(5.f)
			.BorderImage(FAppStyle::GetBrush("ToolTip.Background"))
			.BorderBackgroundColor(FLinearColor(1.f, 1.f, 1.f, 0.75f))
			[
				SNew(SVerticalBox)		
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					CreateSlider()
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				.Padding(0.f, 2.f, 0.f, 0.f)
				[
					CreateControls()
				]
			]
		]
	];
}

void SMediaSourceOverlay::Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(InAllottedGeometry, InCurrentTime, InDeltaTime);

	if (!Container.IsValid())
	{
		return;
	}

	const bool bMouseOver = Delegates.IsValid() ? Delegates->IsOverViewer.Execute() : true;

	if (LastInteractionTime < 0 || bMouseOver)
	{
		LastInteractionTime = InCurrentTime;

		if (Container->GetVisibility() != EVisibility::Visible)
		{
			Container->SetVisibility(EVisibility::Visible);
		}
	}
	else if (InCurrentTime > (LastInteractionTime + FadeTime))
	{
		if (Container->GetVisibility() != EVisibility::Hidden)
		{
			Container->SetVisibility(EVisibility::Hidden);
		}
	}
}

UMediaStream* SMediaSourceOverlay::GetMediaStream() const
{
	if (TSharedPtr<FMediaSourceImageViewer> ImageViewer = ImageViewerWeak.Pin())
	{
		return ImageViewer->GetMediaStream();
	}

	return nullptr;
}

IMediaStreamPlayer* SMediaSourceOverlay::GetMediaStreamPlayer() const
{
	if (UMediaStream* MediaStream = GetMediaStream())
	{
		return MediaStream->GetPlayer().GetInterface();
	}

	return nullptr;
}

UMediaPlayer* SMediaSourceOverlay::GetMediaPlayer() const
{
	if (IMediaStreamPlayer* MediaStreamPlayer = GetMediaStreamPlayer())
	{
		return MediaStreamPlayer->GetPlayer();
	}

	return nullptr;
}

void SMediaSourceOverlay::TrySetFrameRate()
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		// INDEX_NONE here access the currently playing TrackIndex and FormatIndex.
		const float FrameRate = MediaPlayer->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE);

		if (!FMath::IsNearlyZero(FrameRate))
		{
			FrameRateFloat = FrameRate;
		}
	}
}

void SMediaSourceOverlay::BindCommands()
{
	if (!Delegates.IsValid() || !Delegates->GetCommandListForPosition.IsBound())
	{
		return;
	}

	TSharedPtr<FUICommandList> CommandList = Delegates->GetCommandListForPosition.Execute(Position);

	if (!CommandList.IsValid())
	{
		return;
	}

	const FMediaViewerCommands& Commands = FMediaViewerCommands::Get();

	CommandList->MapAction(Commands.QuickRewindVideo, FExecuteAction::CreateSP(this, &SMediaSourceOverlay::AddOffset_Time, -10.f));
	CommandList->MapAction(Commands.RewindVideo, FExecuteAction::CreateSP(this, &SMediaSourceOverlay::AddOffset_Time, -1.f));
	CommandList->MapAction(Commands.StepBackVideo, FExecuteAction::CreateSP(this, &SMediaSourceOverlay::AddOffset_Frame, -1));
	CommandList->MapAction(Commands.ToggleVideoPlay, FExecuteAction::CreateSP(this, &SMediaSourceOverlay::TogglePlay));
	CommandList->MapAction(Commands.StepForwardVideo, FExecuteAction::CreateSP(this, &SMediaSourceOverlay::AddOffset_Frame, 1));
	CommandList->MapAction(Commands.FastForwardVideo, FExecuteAction::CreateSP(this, &SMediaSourceOverlay::AddOffset_Time, 1.f));
	CommandList->MapAction(Commands.QuickFastForwardVideo, FExecuteAction::CreateSP(this, &SMediaSourceOverlay::AddOffset_Time, 10.f));
}

TSharedRef<SWidget> SMediaSourceOverlay::CreateSlider()
{
	UMediaPlayer* MediaPlayer = GetMediaPlayer();

	if (!MediaPlayer)
	{
		return SNullWidget::NullWidget;
	}

	IMediaPlayerEditorModule* MediaPlayerEditorModule = FModuleManager::LoadModulePtr<IMediaPlayerEditorModule>("MediaPlayerEditor");

	if (!MediaPlayerEditorModule)
	{
		return SNullWidget::NullWidget;
	}

	TArray<TWeakObjectPtr<UMediaPlayer>> MediaPlayerList;
	MediaPlayerList.Add(MediaPlayer);

	const TSharedRef<IMediaPlayerSlider> MediaPlayerSlider = MediaPlayerEditorModule->CreateMediaPlayerSliderWidget(MediaPlayerList);

	MediaPlayerSlider->SetSliderHandleColor(FSlateColor(EStyleColor::AccentBlue));
	MediaPlayerSlider->SetVisibleWhenInactive(EVisibility::Visible);
	MediaPlayerSlider->GetScrubEvent().AddSP(this, &SMediaSourceOverlay::OnScrub);

	return MediaPlayerSlider;
}

TSharedRef<SWidget> SMediaSourceOverlay::CreateControls()
{
	const FMediaViewerCommands& MediaViewerCommands = FMediaViewerCommands::Get();
	
	const FText RelatedKeybindsText = LOCTEXT("RelatedKeybinds", "Related Keybinds:");

	TArray<FText> BackwardsKeybinds;
	BackwardsKeybinds.Reserve(4);
	{
		const FText StepBackKeybind = MediaSourceOverlay::GetKeybindText(MediaViewerCommands.StepBackVideo);
		const FText RewindKeybind = MediaSourceOverlay::GetKeybindText(MediaViewerCommands.RewindVideo);
		const FText QuickRewindKeybind = MediaSourceOverlay::GetKeybindText(MediaViewerCommands.QuickRewindVideo);

		if (!StepBackKeybind.IsEmpty() || !RewindKeybind.IsEmpty() || !QuickRewindKeybind.IsEmpty())
		{
			BackwardsKeybinds.Add(RelatedKeybindsText);

			if (!StepBackKeybind.IsEmpty())
			{
				BackwardsKeybinds.Add(FText::Format(
					LOCTEXT("StepBackKeybind", "- Step Back: {0}"),
					StepBackKeybind
				));
			}

			if (!RewindKeybind.IsEmpty())
			{
				BackwardsKeybinds.Add(FText::Format(
					LOCTEXT("RewindKeybind", "- Rewind 1s: {0}"),
					RewindKeybind
				));
			}

			if (!QuickRewindKeybind.IsEmpty())
			{
				BackwardsKeybinds.Add(FText::Format(
					LOCTEXT("QuickRewindKeybind", "- Rewind 10s: {0}"),
					QuickRewindKeybind
				));
			}
		}
	}

	FText StepBackTooltip = LOCTEXT("StepBackward", "Step backward 1 frame.\n\nOnly available while paused.");

	if (!BackwardsKeybinds.IsEmpty())
	{
		StepBackTooltip = FText::Format(
			INVTEXT("{0}\n\n{1}"),
			StepBackTooltip,
			FText::Join(INVTEXT("\n"), BackwardsKeybinds)
		);
	}

	TArray<FText> ForwardsKeybinds;
	ForwardsKeybinds.Reserve(4);
	{
		const FText StepForwardKeybind = MediaSourceOverlay::GetKeybindText(MediaViewerCommands.StepForwardVideo);
		const FText FastForwardKeybind = MediaSourceOverlay::GetKeybindText(MediaViewerCommands.FastForwardVideo);
		const FText QuickFastForwardKeybind = MediaSourceOverlay::GetKeybindText(MediaViewerCommands.QuickFastForwardVideo);

		if (!StepForwardKeybind.IsEmpty() || !FastForwardKeybind.IsEmpty() || !QuickFastForwardKeybind.IsEmpty())
		{
			ForwardsKeybinds.Add(RelatedKeybindsText);

			if (!StepForwardKeybind.IsEmpty())
			{
				ForwardsKeybinds.Add(FText::Format(
					LOCTEXT("StepForwardKeybind", "- Step Forward: {0}"),
					StepForwardKeybind
				));
			}

			if (!FastForwardKeybind.IsEmpty())
			{
				ForwardsKeybinds.Add(FText::Format(
					LOCTEXT("FastForwardKeybind", "- Fast Forward 1s: {0}"),
					FastForwardKeybind
				));
			}

			if (!QuickFastForwardKeybind.IsEmpty())
			{
				ForwardsKeybinds.Add(FText::Format(
					LOCTEXT("QuickFastForwardKeybind", "- Fast Forward 10s: {0}"),
					QuickFastForwardKeybind
				));
			}
		}
	}

	FText StepForwardTooltip = LOCTEXT("StepForward", "Step forward 1 frame.\n\nOnly available while paused.");

	if (!ForwardsKeybinds.IsEmpty())
	{
		StepForwardTooltip = FText::Format(
			INVTEXT("{0}\n\n{1}"),
			StepForwardTooltip,
			FText::Join(INVTEXT("\n"), ForwardsKeybinds)
		);
	}

	return SNew(SHorizontalBox)

		// Current frame
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(0.f, 0.f, 5.f, 0.f)
		.FillWidth(1.f)
		[
			SNew(STextBlock)
			.ColorAndOpacity(FStyleColors::Foreground.GetSpecifiedColor())
			.ShadowColorAndOpacity(FStyleColors::Panel.GetSpecifiedColor())
			.ShadowOffset(FVector2D(1.f, 1.f))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &SMediaSourceOverlay::GetCurrentFrame)
		]

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(0.f, 0.f, 5.f, 0.f)
		[
			SNew(STextBlock)
			.ColorAndOpacity(FStyleColors::Foreground.GetSpecifiedColor())
			.ShadowColorAndOpacity(FStyleColors::Panel.GetSpecifiedColor())
			.ShadowOffset(FVector2D(1.f, 1.f))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(INVTEXT("/"))
		]

		// Total fames
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(0.f, 0.f, 10.f, 0.f)
		[
			SNew(STextBlock)
			.ColorAndOpacity(FStyleColors::Foreground.GetSpecifiedColor())
			.ShadowColorAndOpacity(FStyleColors::Panel.GetSpecifiedColor())
			.ShadowOffset(FVector2D(1.f, 1.f))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &SMediaSourceOverlay::GetTotalFrames)
		]

		// Rewind button.
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaSourceOverlay::Rewind_IsEnabled)
			.OnClicked(this, &SMediaSourceOverlay::Rewind_OnClicked)
			.ButtonStyle(FMediaViewerStyle::Get(), "MediaButtons")
			.ContentPadding(2.f)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Animation.Backward_End").GetIcon())
				.ToolTipText(LOCTEXT("Rewind", "Rewind the media to the beginning"))
				.DesiredSizeOverride(FVector2D(20.f))
			]
		]
		
		// Step back button.
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaSourceOverlay::StepBack_IsEnabled)
			.OnClicked(this, &SMediaSourceOverlay::StepBack_OnClicked)
			.ButtonStyle(FMediaViewerStyle::Get(), "MediaButtons")
			.ContentPadding(2.f)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Animation.Backward_Step").GetIcon())
				.ToolTipText(StepBackTooltip)
				.DesiredSizeOverride(FVector2D(20.f))
			]
		]

		// Reverse button.
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaSourceOverlay::Reverse_IsEnabled)
			.OnClicked(this, &SMediaSourceOverlay::Reverse_OnClicked)
			.ButtonStyle(FMediaViewerStyle::Get(), "MediaButtons")
			.ContentPadding(2.f)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(this, &SMediaSourceOverlay::Reverse_GetBrush)
				.ToolTipText(this, &SMediaSourceOverlay::Reverse_GetToolTip)
				.DesiredSizeOverride(FVector2D(20.f))
			]
		]

		// Play button.
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaSourceOverlay::Play_IsEnabled)
			.OnClicked(this, &SMediaSourceOverlay::Play_OnClicked)
			.ButtonStyle(FMediaViewerStyle::Get(), "MediaButtons")
			.ContentPadding(2.f)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(this, &SMediaSourceOverlay::Play_GetBrush)
				.ToolTipText(this, &SMediaSourceOverlay::Play_GetToolTip)
				.DesiredSizeOverride(FVector2D(20.f))
			]
		]

		// Step forward button.
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaSourceOverlay::StepForward_IsEnabled)
			.OnClicked(this, &SMediaSourceOverlay::StepForward_OnClicked)
			.ButtonStyle(FMediaViewerStyle::Get(), "MediaButtons")
			.ContentPadding(2.f)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Animation.Forward_Step").GetIcon())
				.ToolTipText(StepForwardTooltip)
				.DesiredSizeOverride(FVector2D(20.f))
			]
		]

		// Fast Forward button.
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaSourceOverlay::Forward_IsEnabled)
			.OnClicked(this, &SMediaSourceOverlay::Forward_OnClicked)
			.ButtonStyle(FMediaViewerStyle::Get(), "MediaButtons")
			.ContentPadding(2.f)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Animation.Forward_End").GetIcon())
				.ToolTipText(LOCTEXT("Forward", "Fast forward the media to the end."))
				.DesiredSizeOverride(FVector2D(20.f))
			]
		]

		// Current time
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(10.f, 0.f, 0.f, 0.f)
		[
			SNew(STextBlock)
			.ColorAndOpacity(FStyleColors::Foreground.GetSpecifiedColor())
			.ShadowColorAndOpacity(FStyleColors::Panel.GetSpecifiedColor())
			.ShadowOffset(FVector2D(1.f, 1.f))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &SMediaSourceOverlay::GetCurrentTime)
		]

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(5.f, 0.f, 0.f, 0.f)
		[
			SNew(STextBlock)
			.ColorAndOpacity(FStyleColors::Foreground.GetSpecifiedColor())
			.ShadowColorAndOpacity(FStyleColors::Panel.GetSpecifiedColor())
			.ShadowOffset(FVector2D(1.f, 1.f))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(INVTEXT("/"))
		]

		// Total time
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(5.f, 0.f, 0.f, 0.f)
		.FillWidth(1.f)
		[
			SNew(STextBlock)
			.ColorAndOpacity(FStyleColors::Foreground.GetSpecifiedColor())
			.ShadowColorAndOpacity(FStyleColors::Panel.GetSpecifiedColor())
			.ShadowOffset(FVector2D(1.f, 1.f))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &SMediaSourceOverlay::GetTotalTime)
		];
}

FText SMediaSourceOverlay::GetCurrentFrame() const
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		if (!FrameRateFloat.IsSet())
		{
			const_cast<SMediaSourceOverlay*>(this)->TrySetFrameRate();
		}

		if (FrameRateFloat.IsSet())
		{
			return FText::AsNumber(FMath::FloorToInt(MediaPlayer->GetTime().GetTotalSeconds() * static_cast<double>(FrameRateFloat.GetValue())) + 1);
		}
	}

	return INVTEXT("-");
}

FText SMediaSourceOverlay::GetTotalFrames() const
{
	if (!TotalFrames.IsSet())
	{
		if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
		{
			if (!FrameRateFloat.IsSet())
			{
				const_cast<SMediaSourceOverlay*>(this)->TrySetFrameRate();
			}

			if (FrameRateFloat.IsSet())
			{
				const FTimespan Duration = MediaPlayer->GetDuration();

				TotalFrames = FText::AsNumber(FMath::FloorToInt(Duration.GetTotalSeconds() * static_cast<double>(FrameRateFloat.GetValue())));
			}
		}
	}

	return TotalFrames.Get(INVTEXT("-"));
}

FText SMediaSourceOverlay::GetCurrentTime() const
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		if (!FrameRateFloat.IsSet())
		{
			const_cast<SMediaSourceOverlay*>(this)->TrySetFrameRate();
		}

		if (FrameRateFloat.IsSet())
		{
			// No straight conversion from float to FFrameRate, so estimate like this.
			FFrameRate FrameRate(FMath::RoundToInt(1000000.f * FrameRateFloat.GetValue()), 1000000);
			FTimecode Timecode(MediaPlayer->GetTime().GetTotalSeconds(), FrameRate, /* DropFrame */ false, /* Rollover */ false);

			return FText::FromString(Timecode.ToString());
		}
	}

	return INVTEXT("-");
}

FText SMediaSourceOverlay::GetTotalTime() const
{
	if (!TotalTime.IsSet())
	{
		if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
		{
			if (!FrameRateFloat.IsSet())
			{
				const_cast<SMediaSourceOverlay*>(this)->TrySetFrameRate();
			}

			if (FrameRateFloat.IsSet())
			{
				// No straight conversion from float to FFrameRate, so estimate like this.
				const FFrameRate FrameRate(FMath::RoundToInt(1000000.f * FrameRateFloat.GetValue()), 1000000);
				const FTimecode Timecode(MediaPlayer->GetDuration().GetTotalSeconds(), FrameRate, /* DropFrame */ false, /* Rollover */ false);

				TotalTime = FText::FromString(Timecode.ToString());
			}
		}
	}

	return TotalTime.Get(INVTEXT("-"));
}

bool SMediaSourceOverlay::Rewind_IsEnabled() const
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		return MediaPlayer->IsReady() &&
			MediaPlayer->SupportsSeeking() &&
			MediaPlayer->GetTime() > FTimespan::Zero();
	}

	return false;
}

FReply SMediaSourceOverlay::Rewind_OnClicked()
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		if (Command_Pause(MediaPlayer) && Command_Rewind(MediaPlayer))
		{
			SendEvent(MediaSourceOverlay::Events::Rewind);
		}
	}

	return FReply::Handled();
}

bool SMediaSourceOverlay::Reverse_IsEnabled() const
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		return MediaPlayer->IsReady() && MediaPlayer->SupportsRate(-1.f, /* Unthinned */ true);
	}

	return false;
}

const FSlateBrush* SMediaSourceOverlay::Reverse_GetBrush() const
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		const float Rate = MediaPlayer->GetRate();

		if (Rate < 0 && !FMath::IsNearlyZero(Rate))
		{
			return FAppStyle::Get().GetBrush("Animation.Pause");
		}
	}

	return FAppStyle::Get().GetBrush("Animation.Backward");
}

FText SMediaSourceOverlay::Reverse_GetToolTip() const
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		const float Rate = MediaPlayer->GetRate();

		if (Rate < 0 && !FMath::IsNearlyZero(Rate))
		{
			return LOCTEXT("Reverse_Pause", "Pause media playback");
		}
	}

	return LOCTEXT("Reverse_Reverse", "Play media in reverse.\n\nNot widely supported by media decoders.");
}

FReply SMediaSourceOverlay::Reverse_OnClicked()
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		const float Rate = MediaPlayer->GetRate();

		if (Rate > 0 || FMath::IsNearlyZero(Rate))
		{
			if (Command_Reverse(MediaPlayer))
			{
				SendEvent(MediaSourceOverlay::Events::Reverse);
			}
		}
		else
		{
			if (Command_Pause(MediaPlayer))
			{
				SendEvent(MediaSourceOverlay::Events::Pause);
			}
		}
	}

	return FReply::Handled();
}

bool SMediaSourceOverlay::StepBack_IsEnabled() const
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		return MediaPlayer->IsReady() && MediaPlayer->IsPaused();
	}

	return false;
}

FReply SMediaSourceOverlay::StepBack_OnClicked()
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		if (Command_StepBack(MediaPlayer))
		{
			SendEvent(MediaSourceOverlay::Events::StepBack);
		}
	}

	return FReply::Handled();
}

bool SMediaSourceOverlay::Play_IsEnabled() const
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		return MediaPlayer->IsReady();
	}

	return false;
}

const FSlateBrush* SMediaSourceOverlay::Play_GetBrush() const
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		const float Rate = MediaPlayer->GetRate();

		if (Rate > 0 && !FMath::IsNearlyZero(Rate))
		{
			return FAppStyle::Get().GetBrush("Animation.Pause");
		}
	}

	return FAppStyle::Get().GetBrush("Animation.Forward");
}

FText SMediaSourceOverlay::Play_GetToolTip() const
{
	const FMediaViewerCommands& MediaViewerCommands = FMediaViewerCommands::Get();

	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		const float Rate = MediaPlayer->GetRate();

		if (Rate > 0 && !FMath::IsNearlyZero(Rate))
		{
			return MediaSourceOverlay::AddKeybind(
				LOCTEXT("Play_Pause", "Pause media playback.\n"),
				MediaViewerCommands.ToggleVideoPlay
			);
		}
	}

	return MediaSourceOverlay::AddKeybind(
		LOCTEXT("Play_Play", "Play media forward.\n"),
		MediaViewerCommands.ToggleVideoPlay
	);
}

FReply SMediaSourceOverlay::Play_OnClicked()
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		const float Rate = MediaPlayer->GetRate();

		if (Rate < 0 || FMath::IsNearlyZero(Rate))
		{
			if (Command_Play(MediaPlayer))
			{
				SendEvent(MediaSourceOverlay::Events::Play);
			}
		}
		else
		{
			if (Command_Pause(MediaPlayer))
			{
				SendEvent(MediaSourceOverlay::Events::Pause);
			}
		}
	}

	return FReply::Handled();
}

bool SMediaSourceOverlay::StepForward_IsEnabled() const
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		return MediaPlayer->IsReady() && MediaPlayer->IsPaused();
	}

	return false;
}

FReply SMediaSourceOverlay::StepForward_OnClicked()
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		if (Command_StepForward(MediaPlayer))
		{
			SendEvent(MediaSourceOverlay::Events::StepForward);
		}
	}

	return FReply::Handled();
}

bool SMediaSourceOverlay::Forward_IsEnabled() const
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		return MediaPlayer->IsReady();
	}

	return false;
}

FReply SMediaSourceOverlay::Forward_OnClicked()
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		if (Command_Forward(MediaPlayer))
		{
			SendEvent(MediaSourceOverlay::Events::Forward);
		}
	}

	return FReply::Handled();
}

void SMediaSourceOverlay::SendEvent(FName InEventName)
{
	if (!Delegates.IsValid())
	{
		return;
	}

	if (!Delegates->AreTransformsLocked.IsBound() || !Delegates->AreTransformsLocked.Execute())
	{
		return;
	}

	FMediaImageViewerEventParams Params;
	Params.FromViewer = ImageViewerWeak.Pin();
	Params.EventName = InEventName;

	Delegates->ImageViewerEvent.Broadcast(Params);
}

void SMediaSourceOverlay::ReceiveEvent(const FMediaImageViewerEventParams& InEventParams)
{
	TSharedPtr<FMediaSourceImageViewer> ImageViewer = ImageViewerWeak.Pin();

	if (!ImageViewer.IsValid())
	{
		return;
	}

	if (InEventParams.FromViewer.Get() == ImageViewer.Get())
	{
		return;
	}

	UMediaPlayer* MediaPlayer = GetMediaPlayer();

	if (!MediaPlayer)
	{
		return;
	}

	using namespace MediaSourceOverlay;

	if (InEventParams.EventName == Events::Rewind)
	{
		Command_Rewind(MediaPlayer);
	}
	else if (InEventParams.EventName == Events::Reverse)
	{
		Command_Reverse(MediaPlayer);
	}
	else if (InEventParams.EventName == Events::StepBack)
	{
		Command_StepBack(MediaPlayer);
	}
	else if (InEventParams.EventName == Events::Play)
	{
		Command_Play(MediaPlayer);
	}
	else if (InEventParams.EventName == Events::Pause)
	{
		Command_Pause(MediaPlayer);
	}
	else if (InEventParams.EventName == Events::StepForward)
	{
		Command_StepForward(MediaPlayer);
	}
	else if (InEventParams.EventName == Events::Forward)
	{
		Command_Forward(MediaPlayer);
	}
	else if (InEventParams.EventName == Events::Scrub)
	{
		ReceiveScrub(static_cast<const MediaSourceOverlay::Events::FMediaImageViewerScrubEventParams&>(InEventParams));
	}
}

void SMediaSourceOverlay::OnScrub(IMediaPlayerSlider::EScrubEventType InSliderEvent, TConstArrayView<UMediaPlayer*> InMediaPlayers, float InScrubPosition)
{
	if (InMediaPlayers.Num() != 1 || !IsValid(InMediaPlayers[0]))
	{
		return;
	}

	UMediaPlayer* MediaPlayer = InMediaPlayers[0];

	if (!Delegates.IsValid())
	{
		return;
	}

	if (!Delegates->AreTransformsLocked.IsBound() || !Delegates->GetSettings.IsBound())
	{
		return;
	}

	if (!Delegates->AreTransformsLocked.Execute())
	{
		return;
	}

	using namespace MediaSourceOverlay::Events;

	const EMediaViewerMediaSyncType MediaSyncType = Delegates->GetSettings.Execute()->MediaSyncType;

	FMediaImageViewerScrubEventParams Params;
	Params.FromViewer = ImageViewerWeak.Pin();
	Params.EventName = Scrub;
	Params.ScrubEvent = InSliderEvent;

	auto SetScrubData = [this, MediaPlayer, MediaSyncType, InScrubPosition, &Params]()
		{
			switch (MediaSyncType)
			{
				case EMediaViewerMediaSyncType::RelativeTime:
					if (!SyncedScrubStartTime.IsType<FTimespan>())
					{
						return false;
					}

					Params.ScrubData.Set<FTimespan>((MediaPlayer->GetDuration() * InScrubPosition) - SyncedScrubStartTime.Get<FTimespan>());
					break;

				case EMediaViewerMediaSyncType::AbsoluteTime:
					Params.ScrubData.Set<FTimespan>(MediaPlayer->GetDuration() * InScrubPosition);
					break;

				case EMediaViewerMediaSyncType::RelativeProgress:
					if (!SyncedScrubStartTime.IsType<float>())
					{
						return false;
					}

					Params.ScrubData.Set<float>(InScrubPosition - SyncedScrubStartTime.Get<float>());
					break;

				case EMediaViewerMediaSyncType::AbsoluteProgress:
					Params.ScrubData.Set<float>(InScrubPosition);
					break;

				default:
					return false;
			}			

			return true;
		};

	switch (InSliderEvent)
	{
		case IMediaPlayerSlider::EScrubEventType::Begin:
			switch (MediaSyncType)
			{
				case EMediaViewerMediaSyncType::RelativeTime:
					SyncedScrubStartTime.Set<FTimespan>(MediaPlayer->GetDuration() * InScrubPosition);
					break;

				case EMediaViewerMediaSyncType::RelativeProgress:
					SyncedScrubStartTime.Set<float>(InScrubPosition);
					break;

				default:
					// Nothing
					break;
			};			

			Delegates->ImageViewerEvent.Broadcast(Params);
			break;

		case IMediaPlayerSlider::EScrubEventType::Update:
			if (SetScrubData())
			{
				Delegates->ImageViewerEvent.Broadcast(Params);
			}
			break;

		case IMediaPlayerSlider::EScrubEventType::End:
			if (SetScrubData())
			{
				Delegates->ImageViewerEvent.Broadcast(Params);
			}

			SyncedScrubStartTime = {};
			break;

		default:
			// Do nothing
			break;
	}
}

void SMediaSourceOverlay::ReceiveScrub(const MediaSourceOverlay::Events::FMediaImageViewerScrubEventParams& InEventParams)
{
	TSharedPtr<FMediaSourceImageViewer> ImageViewer = ImageViewerWeak.Pin();

	if (!ImageViewer.IsValid())
	{
		return;
	}

	using namespace MediaSourceOverlay::Events;

	if (InEventParams.FromViewer.Get() == ImageViewer.Get())
	{
		return;
	}

	UMediaPlayer* MediaPlayer = GetMediaPlayer();

	if (!MediaPlayer)
	{
		return;
	}

	if (!Delegates.IsValid())
	{
		return;
	}

	if (!Delegates->GetSettings.IsBound())
	{
		return;
	}

	if (!MediaPlayer->SupportsScrubbing())
	{
		return;
	}

	const EMediaViewerMediaSyncType MediaSyncType = Delegates->GetSettings.Execute()->MediaSyncType;

	auto UpdateScrub = [this, MediaPlayer, MediaSyncType, &InEventParams]()
		{
			switch (MediaSyncType)
			{
				case EMediaViewerMediaSyncType::RelativeTime:
					if (SyncedScrubStartTime.IsType<FTimespan>() && InEventParams.ScrubData.IsType<FTimespan>())
					{
						MediaPlayer->Scrub(FTimespan::FromSeconds(FMath::Clamp(
							(SyncedScrubStartTime.Get<FTimespan>() + InEventParams.ScrubData.Get<FTimespan>()).GetTotalSeconds(),
							0.0,
							MediaPlayer->GetDuration().GetTotalSeconds()
						)));
					}
					break;

				case EMediaViewerMediaSyncType::AbsoluteTime:
					if (InEventParams.ScrubData.IsType<FTimespan>())
					{
						MediaPlayer->Scrub(FTimespan::FromSeconds(FMath::Clamp(
							InEventParams.ScrubData.Get<FTimespan>().GetTotalSeconds(),
							0.0,
							MediaPlayer->GetDuration().GetTotalSeconds()
						)));
					}
					break;

				case EMediaViewerMediaSyncType::RelativeProgress:
					if (SyncedScrubStartTime.IsType<float>() && InEventParams.ScrubData.IsType<float>())
					{
						MediaPlayer->Scrub(MediaPlayer->GetDuration() * FMath::Clamp(
							SyncedScrubStartTime.Get<float>() + InEventParams.ScrubData.Get<float>(),
							0.0,
							1.0
						));
					}
					break;

				case EMediaViewerMediaSyncType::AbsoluteProgress:
					if (InEventParams.ScrubData.IsType<float>())
					{
						MediaPlayer->Scrub(MediaPlayer->GetDuration() * FMath::Clamp(
							InEventParams.ScrubData.Get<float>(),
							0.0,
							1.0
						));
					}
					break;

				default:
					// Do nothing
					break;
			}
		};

	switch (InEventParams.ScrubEvent)
	{
		case IMediaPlayerSlider::EScrubEventType::Begin:
		{
			if (MediaPlayer->IsPlaying())
			{
				RateWhenScrubEventReceived = MediaPlayer->GetRate();
				Command_Pause(MediaPlayer);
			}

			switch (MediaSyncType)
			{
				case EMediaViewerMediaSyncType::RelativeTime:
					SyncedScrubStartTime.Set<FTimespan>(MediaPlayer->GetTime());
					break;

				case EMediaViewerMediaSyncType::RelativeProgress:
				{
					const float MediaDuration = MediaPlayer->GetDuration().GetTotalSeconds();

					if (MediaDuration > 0)
					{
						SyncedScrubStartTime.Set<float>(MediaPlayer->GetTime().GetTotalSeconds() / MediaDuration);
					}

					break;
				}

				default:
					// Nothing
					break;
			};

			break;
		}

		case IMediaPlayerSlider::EScrubEventType::Update:
		{
			UpdateScrub();
			break;
		}

		case IMediaPlayerSlider::EScrubEventType::End:
		{
			UpdateScrub();
			SyncedScrubStartTime = {};

			if (RateWhenScrubEventReceived.IsSet())
			{
				MediaPlayer->SetRate(RateWhenScrubEventReceived.GetValue());
				RateWhenScrubEventReceived = {};
			}

			break;
		}

		default:
			// Do nothing
			break;
	}
}

bool SMediaSourceOverlay::Command_Rewind(UMediaPlayer* InMediaPlayer)
{
	InMediaPlayer->SetRate(0.f);
	InMediaPlayer->Rewind();

	return true;
}

bool SMediaSourceOverlay::Command_Reverse(UMediaPlayer* InMediaPlayer)
{
	InMediaPlayer->SetRate(-1.f);

	return true;
}

bool SMediaSourceOverlay::Command_StepBack(UMediaPlayer* InMediaPlayer)
{
	const float FrameRate = InMediaPlayer->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE);

	if (!FMath::IsNearlyZero(FrameRate))
	{
		InMediaPlayer->Seek(InMediaPlayer->GetTime() - FTimespan::FromSeconds(1.f / FrameRate));

		return true;
	}

	return false;
}

bool SMediaSourceOverlay::Command_Play(UMediaPlayer* InMediaPlayer)
{
	InMediaPlayer->SetRate(1.f);

	return true;
}

bool SMediaSourceOverlay::Command_Pause(UMediaPlayer* InMediaPlayer)
{
	InMediaPlayer->SetRate(0.f);

	return true;
}

bool SMediaSourceOverlay::Command_StepForward(UMediaPlayer* InMediaPlayer)
{
	const float FrameRate = InMediaPlayer->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE);

	if (!FMath::IsNearlyZero(FrameRate))
	{
		InMediaPlayer->Seek(InMediaPlayer->GetTime() + FTimespan::FromSeconds(1.f / FrameRate));

		return true;
	}

	return false;
}

bool SMediaSourceOverlay::Command_Forward(UMediaPlayer* InMediaPlayer)
{
	// INDEX_NONE here access the currently playing TrackIndex and FormatIndex.
	const double FrameRate = InMediaPlayer->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE);

	if (!FMath::IsNearlyZero(FrameRate))
	{
		InMediaPlayer->SetRate(0.f);

		const double FrameTime = 1.0 / FrameRate;
		FTimespan SeekLocation = InMediaPlayer->GetDuration();
		SeekLocation -= FTimespan::FromSeconds(FrameTime);

		InMediaPlayer->Seek(SeekLocation);

		return true;
	}

	return false;
}

void SMediaSourceOverlay::AddOffset_Time(float InOffset)
{
	UMediaPlayer* MediaPlayer = GetMediaPlayer();

	if (!MediaPlayer)
	{
		return;
	}

	const float Duration = MediaPlayer->GetDuration().GetTotalSeconds();

	if (FMath::IsNearlyZero(Duration))
	{
		return;
	}

	const float CurrentPosition = MediaPlayer->GetTime().GetTotalSeconds();

	if (CurrentPosition < 0 || CurrentPosition > Duration)
	{
		return;
	}

	const float NewPosition = FMath::Clamp(CurrentPosition + InOffset, 0, Duration);

	if (FMath::IsNearlyEqual(CurrentPosition, NewPosition))
	{
		return;
	}

	MediaPlayer->Seek(FTimespan::FromSeconds(NewPosition));

	// Broadcast the event if required by simulating a scrub.
	const float StartFraction = CurrentPosition / Duration;
	const float EndFraction = NewPosition / Duration;

	TArray<UMediaPlayer*, TInlineAllocator<1>> MediaPlayers = {MediaPlayer};	

	OnScrub(IMediaPlayerSlider::EScrubEventType::Begin, MediaPlayers, StartFraction);
	OnScrub(IMediaPlayerSlider::EScrubEventType::Update, MediaPlayers, EndFraction);
	OnScrub(IMediaPlayerSlider::EScrubEventType::End, MediaPlayers, EndFraction);
}

void SMediaSourceOverlay::AddOffset_Frame(int32 InOffset)
{
	switch (InOffset)
	{
		case 1:
			if (StepForward_IsEnabled())
			{
				StepForward_OnClicked();
			}
			break;

		case -1:
			if (StepBack_IsEnabled())
			{
				StepBack_OnClicked();
			}
			break;
	}
}

void SMediaSourceOverlay::TogglePlay()
{
	if (Play_IsEnabled())
	{
		Play_OnClicked();
	}
}

}

#undef LOCTEXT_NAMESPACE
