// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Internationalization/Text.h"
#include "MediaPlayerEditorModule.h"
#include "Misc/Optional.h"
#include "Misc/Timespan.h"
#include "Misc/TVariant.h"

class IMediaStreamPlayer;
class SBox;
class UMediaPlayer;
class UMediaStream;
struct FSlateBrush;

namespace UE::MediaViewer
{
	class SMediaViewerTab;
	enum class EMediaImageViewerPosition : uint8;
	struct FMediaImageViewerEventParams;
	struct FMediaViewerDelegates;
}

namespace UE::MediaViewer::Private
{

class FMediaSourceImageViewer;

namespace MediaSourceOverlay::Events
{
	struct FMediaImageViewerScrubEventParams;
}
	
class SMediaSourceOverlay : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SMediaSourceOverlay, SCompoundWidget)

	SLATE_BEGIN_ARGS(SMediaSourceOverlay)
		{}
	SLATE_END_ARGS()

public:
	virtual ~SMediaSourceOverlay() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<FMediaSourceImageViewer>& InImageViewer, EMediaImageViewerPosition InPosition,
		const TSharedPtr<FMediaViewerDelegates>& InDelegates);

	//~ Begin SWidget
	virtual void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget

protected:
	/** The image viewer being controlled by this overlay. */
	TWeakPtr<FMediaSourceImageViewer> ImageViewerWeak;

	/** The position in the viewer occupied by this image viewer. */
	EMediaImageViewerPosition Position;

	/** The delegates used to communicate with other widgets. */
	TSharedPtr<FMediaViewerDelegates> Delegates;

	/** Container so the overlay can be toggled on and off. */
	TSharedPtr<SBox> Container;

	/** The frame rate stored before a scrub-based pause event. */
	TOptional<float> FrameRateFloat;

	mutable TOptional<FText> TotalFrames;
	mutable TOptional<FText> TotalTime;

	/** When a sync scrub is started, this stores the start of this media. */
	TVariant<FTimespan, float> SyncedScrubStartTime;
	TOptional<float> RateWhenScrubEventReceived;

	/** The last time the mouse was moved over the widget. */
	double LastInteractionTime = -1.0;

	UMediaStream* GetMediaStream() const;

	IMediaStreamPlayer* GetMediaStreamPlayer() const;

	UMediaPlayer* GetMediaPlayer() const;

	/** Attempt to store the frame rate of the currently playing media. */
	void TrySetFrameRate();

	void BindCommands();

	TSharedRef<SWidget> CreateSlider();

	/** Create play controls. Play, pause, etc. */
	TSharedRef<SWidget> CreateControls();

	FText GetCurrentFrame() const;
	FText GetTotalFrames() const;

	FText GetCurrentTime() const;
	FText GetTotalTime() const;

	bool Rewind_IsEnabled() const;
	FReply Rewind_OnClicked();

	bool Reverse_IsEnabled() const;
	const FSlateBrush* Reverse_GetBrush() const;
	FText Reverse_GetToolTip() const;
	FReply Reverse_OnClicked();

	bool StepBack_IsEnabled() const;
	FReply StepBack_OnClicked();

	bool Play_IsEnabled() const;
	const FSlateBrush* Play_GetBrush() const;
	FText Play_GetToolTip() const;
	FReply Play_OnClicked();

	bool StepForward_IsEnabled() const;
	FReply StepForward_OnClicked();

	bool Forward_IsEnabled() const;
	FReply Forward_OnClicked();

	/** Broadcasts an event that can be received by others. */
	void SendEvent(FName InEventName);

	/** Receive an event broadcast by somebody else. */
	void ReceiveEvent(const FMediaImageViewerEventParams& InEventParams);

	/** Triggered when this player's scrub bar is manually moved. */
	void OnScrub(IMediaPlayerSlider::EScrubEventType InScrubEvent, TConstArrayView<UMediaPlayer*> InMediaPlayers, float InScrubPosition);

	/** Execute commands on the given media player. */
	static bool Command_Rewind(UMediaPlayer* InMediaPlayer);
	static bool Command_Reverse(UMediaPlayer* InMediaPlayer);
	static bool Command_StepBack(UMediaPlayer* InMediaPlayer);
	static bool Command_Play(UMediaPlayer* InMediaPlayer);
	static bool Command_Pause(UMediaPlayer* InMediaPlayer);
	static bool Command_StepForward(UMediaPlayer* InMediaPlayer);
	static bool Command_Forward(UMediaPlayer* InMediaPlayer);
	
	/** Triggered when a scrub event is received from someone else. */
	void ReceiveScrub(const MediaSourceOverlay::Events::FMediaImageViewerScrubEventParams& InEventParams);

	/** Add a offset to the current media. Optionally broadcasts it to others as a scrub event. */
	void AddOffset_Time(float InOffset);

	/** Add a offset to the current media. Optionally broadcasts it to others as a scrub event. Supports 1 and -1. */
	void AddOffset_Frame(int32 InOffset);

	/** Toggles between play and pause. Forward only. */
	void TogglePlay();
};

} // UE::MediaViewer::Private