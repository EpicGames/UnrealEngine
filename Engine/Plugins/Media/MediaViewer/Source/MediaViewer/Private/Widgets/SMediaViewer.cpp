// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaViewer.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "CanvasTypes.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IContentBrowserSingleton.h"
#include "ImageViewer/MediaImageViewer.h"
#include "ImageViewers/NullImageViewer.h"
#include "ImageViewers/Texture2DImageViewer.h"
#include "IMediaViewerModule.h"
#include "Library/MediaViewerLibrary.h"
#include "Library/MediaViewerLibraryGroup.h"
#include "Library/MediaViewerLibraryIni.h"
#include "MediaViewerCommands.h"
#include "MediaViewerDelegates.h"
#include "MediaViewerUtils.h"
#include "Misc/MessageDialog.h"
#include "PackageTools.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "Sidebar/ISidebarDrawerContent.h"
#include "Sidebar/SSidebar.h"
#include "Sidebar/SSidebarContainer.h"
#include "TextureResource.h"
#include "UObject/Package.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SMediaImageViewerOverlay.h"
#include "Widgets/SMediaViewerLibrary.h"
#include "Widgets/SMediaViewerLibraryPrivate.h"
#include "Widgets/SMediaViewerToolbar.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SMediaViewer)

#define LOCTEXT_NAMESPACE "SMediaViewer"

namespace UE::MediaViewer::Private
{

static const FLazyName DrawerId = "Library";

class FLibraryDrawerConfig : public ISidebarDrawerContent
{
public:
	FLibraryDrawerConfig(TSharedRef<SMediaViewerLibraryPrivate> InLibraryWidget)
		: LibraryWidget(InLibraryWidget)
	{
	}

	virtual FName GetUniqueId() const override
	{
		return DrawerId;
	}

	virtual FName GetSectionId() const
	{
		return DrawerId;
	}

	virtual FText GetSectionDisplayText() const
	{
		return LOCTEXT("Library", "Library");
	}

	virtual bool ShouldShowSection() const
	{
		return true;
	}

	virtual TOptional<float> GetSectionFillHeight() const
	{
		return 1.f;
	}

	virtual int32 GetSortOrder() const
	{
		return 0;
	}

	virtual TSharedRef<SWidget> CreateContentWidget()
	{
		return LibraryWidget;
	}

private:
	TSharedRef<SMediaViewerLibraryPrivate> LibraryWidget;
};

const FSlateColorBrush SMediaViewer::BackgroundColorBrush(FLinearColor::Black);

SMediaViewer::SMediaViewer()
	: CommandList(MakeShared<FUICommandList>())
	, BackgroundTextureBrush(static_cast<UObject*>(nullptr), FVector2D(1), FLinearColor::White, ESlateBrushTileType::NoTile)
	, ViewerSize(FVector2D::ZeroVector)
	, ViewerPosition(FVector2D::ZeroVector)
	, bInvalidated(true)
{
	FWorldDelegates::OnWorldCleanup.AddRaw(this, &SMediaViewer::OnWorldCleanUp);
	FEditorDelegates::OnPackageDeleted.AddRaw(this, &SMediaViewer::OnPackageDeleted);
	FEditorDelegates::OnAssetsPreDelete.AddRaw(this, &SMediaViewer::OnAssetsDeleted);
}

SMediaViewer::~SMediaViewer()
{
	FWorldDelegates::OnWorldCleanup.RemoveAll(this);
	FEditorDelegates::OnPackageDeleted.RemoveAll(this);
	FEditorDelegates::OnAssetsPreDelete.RemoveAll(this);
}

void SMediaViewer::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

void SMediaViewer::Construct(const FArguments& InArgs, const TSharedRef<SMediaViewerTab>& InTab, const FMediaViewerArgs& InMediaViewerArgs,
	const TSharedRef<FMediaImageViewer>& InImageViewerFirst, const TSharedRef<FMediaImageViewer>& InImageViewerSecond)
{
	// Warm shader caches now so the editor isn't stalled the first time we render with
	// these materials in user-facing paths. No-op after the first call.
	UE::MediaViewer::Private::FMediaViewerUtils::PreloadMaterialAssets();

	MediaViewerArgs = InMediaViewerArgs;
	Tab = InTab;
	ImageViewers[static_cast<int32>(EMediaImageViewerPosition::First)] = InImageViewerFirst;
	ImageViewers[static_cast<int32>(EMediaImageViewerPosition::Second)] = InImageViewerSecond;
	ActiveView = EMediaImageViewerActivePosition::Single;
	RequestedView = EMediaImageViewerActivePosition::Single;
	ScaleToFit[static_cast<int32>(EMediaImageViewerPosition::First)] = false;
	ScaleToFit[static_cast<int32>(EMediaImageViewerPosition::Second)] = false;
	ContentSlot = nullptr;
	CursorLocalPosition = {0, 0};

	BindCommands();
	CreateDelegates();

	Library = SNew(SMediaViewerLibraryPrivate, Delegates.ToSharedRef());

	Layout = SNew(SVerticalBox);

	if (MediaViewerArgs.bShowToolbar)
	{
		Layout->AddSlot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.AutoHeight()
			.Expose(ToolbarSlot)
			[
				SNullWidget::NullWidget
			];
	}

	Layout->AddSlot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.FillHeight(1.f)
		.Expose(ContentSlot)
		[
			SNullWidget::NullWidget
		];

	if (!MediaViewerArgs.bShowSidebar)
	{
		ChildSlot
		[
			Layout.ToSharedRef()
		];

		CreateView();
		return;
	}

	TSharedRef<SSidebarContainer> SidebarContainer = SNew(SSidebarContainer);

	TSharedRef<SSidebar> Sidebar = SNew(SSidebar, SidebarContainer)
		.TabLocation(ESidebarTabLocation::Left)
		.InitialDrawerSize(0.25f)
		.OnGetContent(FOnGetContent::CreateSPLambda(
			this,
			[this]
			{ 
				return Layout.ToSharedRef();
			})
		);

	FSidebarState SidebarState;
	SidebarState.SetDrawerSizes(0.25f, 0.25f);

	SidebarContainer->RebuildSidebar(Sidebar, SidebarState);

	FSidebarDrawerConfig LibraryDrawerConfig;
	LibraryDrawerConfig.UniqueId = DrawerId;
	LibraryDrawerConfig.ButtonText = LOCTEXT("Library", "Library");
	LibraryDrawerConfig.ToolTipText = LOCTEXT("LibraryTooltip", "Open the Library side bar.");
	LibraryDrawerConfig.Icon = FAppStyle::GetBrush("Icons.FolderOpen");
	LibraryDrawerConfig.InitialState = SidebarState.FindOrAddDrawerState(DrawerId.Resolve());

	Sidebar->RegisterDrawer(MoveTemp(LibraryDrawerConfig));
	Sidebar->RegisterDrawerSection(DrawerId, MakeShared<FLibraryDrawerConfig>(Library.ToSharedRef()));

	ChildSlot
	[
		SidebarContainer
	];

	CreateView();
}

TSharedRef<IMediaViewerLibrary> SMediaViewer::GetLibrary() const
{
	return Library->GetLibrary();
}

TSharedPtr<FMediaImageViewer> SMediaViewer::GetImageViewer(EMediaImageViewerPosition InPosition) const
{
	return ImageViewers[static_cast<int32>(InPosition)];
}

void SMediaViewer::SetImageViewer(EMediaImageViewerPosition InPosition, const TSharedRef<FMediaImageViewer>& InImageViewer)
{
	const int32 Index = static_cast<int32>(InPosition);

	if (ImageViewers[Index].IsValid() && ImageViewers[Index]->GetInfo().Id == InImageViewer->GetInfo().Id)
	{
		return;
	}

	ImageViewers[Index] = InImageViewer;
	ScaleToFit[Index] = true;

	if (InImageViewer != FNullImageViewer::GetNullImageViewer())
	{
		Library->OnImageViewerOpened(InImageViewer);
	}

	if (InPosition == EMediaImageViewerPosition::Second)
	{
		RequestedView = EMediaImageViewerActivePosition::Both;
	}

	InvalidateView();
}

void SMediaViewer::ClearImageViewer(EMediaImageViewerPosition InPosition)
{
	SetImageViewer(InPosition, FNullImageViewer::GetNullImageViewer());
}

void SMediaViewer::BindCommands()
{
	const FMediaViewerCommands& Commands = FMediaViewerCommands::Get();

	CommandList->MapAction(
		Commands.ToggleOverlay, 
		FExecuteAction::CreateSP(this, &SMediaViewer::ToggleOverlays),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSP(this, &SMediaViewer::AreOverlaysEnabled)
	);

	CommandList->MapAction(
		Commands.ToggleLockedTransform,
		FExecuteAction::CreateSP(this, &SMediaViewer::ToggleLockedTransform),
		FCanExecuteAction::CreateSP(this, &SMediaViewer::IsShowingBothImageViewers),
		FGetActionCheckState::CreateSP(this, &SMediaViewer::AreLockedTransformEnabled)
	);
	
	CommandList->MapAction(
		Commands.SecondImageOpacityMinus,
		FExecuteAction::CreateSP(this, &SMediaViewer::AdjustSecondImageOpacity, -10.f),
		FCanExecuteAction::CreateSP(this, &SMediaViewer::IsShowingBothImageViewers)
	);
	
	CommandList->MapAction(
		Commands.SecondImageOpacityPlus, 
		FExecuteAction::CreateSP(this, &SMediaViewer::AdjustSecondImageOpacity, 10.f),
		FCanExecuteAction::CreateSP(this, &SMediaViewer::IsShowingBothImageViewers)
	);

	CommandList->MapAction(
		Commands.SwapAB, 
		FExecuteAction::CreateSP(this, &SMediaViewer::SwapABImageViewers),
		FCanExecuteAction::CreateSP(this, &SMediaViewer::IsShowingBothImageViewers)
	);

	CommandList->MapAction(Commands.ResetAllTransforms, FExecuteAction::CreateSP(this, &SMediaViewer::ResetTransformToAll));

	CommandList->MapAction(
		Commands.Snapshot,
		FExecuteAction::CreateSP(this, &SMediaViewer::TakeSnapshot),
		FCanExecuteAction::CreateSP(this, &SMediaViewer::CanTakeSnapshot)
	);

	CommandList->MapAction(Commands.OpenNewTab, FExecuteAction::CreateSP(this, &SMediaViewer::OpenNewTab));

	CommandList->MapAction(
		Commands.MirrorHorizonal,
		FExecuteAction::CreateSP(this, &SMediaViewer::Mirror, EMediaImageMirrorPlane::Horizontal)
	);

	CommandList->MapAction(
		Commands.MirrorVertical,
		FExecuteAction::CreateSP(this, &SMediaViewer::Mirror, EMediaImageMirrorPlane::Vertical)
	);

	CommandList->MapAction(
		Commands.MirrorDepth,
		FExecuteAction::CreateSP(this, &SMediaViewer::Mirror, EMediaImageMirrorPlane::Depth)
	);

	auto BindBookmark = [this](const TSharedPtr<FUICommandInfo>& InOpenCommand, const TSharedPtr<FUICommandInfo>& InSaveCommand, int32 InIndex)
		{
			CommandList->MapAction(
				InOpenCommand,
				FExecuteAction::CreateSP(this, &SMediaViewer::OpenBookmark, InIndex),
				FCanExecuteAction::CreateSP(this, &SMediaViewer::CanOpenBookmark, InIndex)
			);

			CommandList->MapAction(
				InSaveCommand,
				FExecuteAction::CreateSP(this, &SMediaViewer::SaveBookmark, InIndex),
				FCanExecuteAction::CreateSP(this, &SMediaViewer::CanSaveBookmark)
			);
		};

	BindBookmark(Commands.OpenBookmark1, Commands.SaveBookmark1, 0);
	BindBookmark(Commands.OpenBookmark2, Commands.SaveBookmark2, 1);
	BindBookmark(Commands.OpenBookmark3, Commands.SaveBookmark3, 2);
	BindBookmark(Commands.OpenBookmark4, Commands.SaveBookmark4, 3);
	BindBookmark(Commands.OpenBookmark5, Commands.SaveBookmark5, 4);
	BindBookmark(Commands.OpenBookmark6, Commands.SaveBookmark6, 5);
	BindBookmark(Commands.OpenBookmark7, Commands.SaveBookmark7, 6);
	BindBookmark(Commands.OpenBookmark8, Commands.SaveBookmark8, 7);
	BindBookmark(Commands.OpenBookmark9, Commands.SaveBookmark9, 8);
	BindBookmark(Commands.OpenBookmark10, Commands.SaveBookmark10, 9);
}

bool SMediaViewer::IsShowingBothImageViewers() const
{
	return ActiveView == EMediaImageViewerActivePosition::Both;
}

void SMediaViewer::SwapABImageViewers()
{
	TSharedPtr<FMediaImageViewer> OldSecond = ImageViewers[static_cast<int32>(EMediaImageViewerPosition::Second)];	
	ImageViewers[static_cast<int32>(EMediaImageViewerPosition::Second)] = ImageViewers[static_cast<int32>(EMediaImageViewerPosition::First)];
	ImageViewers[static_cast<int32>(EMediaImageViewerPosition::First)] = OldSecond;

	const FVector OldOffset = ImageViewers[static_cast<int32>(EMediaImageViewerPosition::First)]->GetPaintSettings().Offset;
	ImageViewers[static_cast<int32>(EMediaImageViewerPosition::First)]->GetPaintSettings().Offset = ImageViewers[static_cast<int32>(EMediaImageViewerPosition::Second)]->GetPaintSettings().Offset;
	ImageViewers[static_cast<int32>(EMediaImageViewerPosition::Second)]->GetPaintSettings().Offset = OldOffset;

	InvalidateView();
}

void SMediaViewer::SetSingleView()
{
	if (ActiveView == EMediaImageViewerActivePosition::Single)
	{
		return;
	}

	RequestedView = EMediaImageViewerActivePosition::Single;

	InvalidateView();
}

void SMediaViewer::SetABView()
{
	if (ActiveView == EMediaImageViewerActivePosition::Both)
	{
		return;
	}

	RequestedView = EMediaImageViewerActivePosition::Both;

	InvalidateView();
}

void SMediaViewer::SetABOrientation(EOrientation InOrientation)
{
	if (MediaViewerSettings.ABOrientation == InOrientation
		&& ActiveView == EMediaImageViewerActivePosition::Both)
	{
		return;
	}

	MediaViewerSettings.ABOrientation = InOrientation;
	RequestedView = EMediaImageViewerActivePosition::Both;
	InvalidateView();
}

void SMediaViewer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	const FVector2D ScreenSpacePosition = FSlateApplication::Get().GetCursorPos();
	const FVector2D LocalPosition = AllottedGeometry.AbsoluteToLocal(ScreenSpacePosition);

	bool bNewLocation = (CursorLocalPosition - LocalPosition).IsNearlyZero() == false;

	if (bNewLocation)
	{
		CursorLocalPosition = LocalPosition;
	}

	constexpr int32 TickCheckValue = 2;
	constexpr int32 TickStopCheckValue = 3;

	if (TickCount < TickStopCheckValue)
	{
		++TickCount;

		if (TickCount == TickCheckValue)
		{
			CheckLastOpenedState();
		}
	}

	if (bInvalidated)
	{
		CreateView();
		return;
	}

	const TOptional<FVector2D> UpdatedMousePosition = bNewLocation
		? LocalPosition
		: TOptional<FVector2D>();
	
	if (TSharedPtr<SMediaImageViewerOverlay> FirstOverlay = GetOverlay(EMediaImageViewerPosition::First))
	{
		FirstOverlay->UpdateMouse(UpdatedMousePosition);
	}

	if (TSharedPtr<SMediaImageViewerOverlay> SecondOverlay = GetOverlay(EMediaImageViewerPosition::Second))
	{
		SecondOverlay->UpdateMouse(UpdatedMousePosition);
	}
}

int32 SMediaViewer::OnPaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InMyCullingRect,
	FSlateWindowElementList& InOutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const
{
	const FSlateLayoutTransform ViewerTransform = FSlateLayoutTransform(ViewerPosition);
	const FPaintGeometry ViewerPaintGeometry = InAllottedGeometry.ToPaintGeometry(ViewerSize, ViewerTransform);

	FSlateDrawElement::MakeBox(
		InOutDrawElements,
		InLayerId,
		ViewerPaintGeometry,
		&BackgroundColorBrush,
		ESlateDrawEffect::NoPixelSnapping,
		MediaViewerSettings.ClearColor
	);

	++InLayerId;

	if (UTexture* BackgroundTexture = MediaViewerSettings.Texture.LoadSynchronous())
	{
		BackgroundTextureBrush.SetResourceObject(BackgroundTexture);
		BackgroundTextureBrush.ImageSize.X = BackgroundTexture->GetSurfaceWidth();
		BackgroundTextureBrush.ImageSize.Y = BackgroundTexture->GetSurfaceHeight();

		const FVector2D ImageSize = BackgroundTextureBrush.ImageSize * MediaViewerSettings.Scale;

		TArray<FSlateVertex> Verts;
		Verts.Reserve(4);

		FSlateVertex BaseVert;
		BaseVert.Color = FColor::White;
		BaseVert.SecondaryColor = FColor::White;
		BaseVert.Position[0] = ViewerPosition.X;
		BaseVert.Position[1] = ViewerPosition.Y;
		BaseVert.TexCoords[0] = 0.f;
		BaseVert.TexCoords[1] = 0.f;
		BaseVert.TexCoords[2] = 1.f;
		BaseVert.TexCoords[3] = 1.f;

		FSlateVertex& TopLeft = Verts.Add_GetRef(BaseVert);

		FSlateVertex& TopRight = Verts.Add_GetRef(BaseVert);
		TopRight.TexCoords[0] = 1.f;

		FSlateVertex& BottomLeft = Verts.Add_GetRef(BaseVert);
		BottomLeft.TexCoords[1] = 1.f;

		FSlateVertex& BottomRight = Verts.Add_GetRef(BaseVert);
		BottomRight.TexCoords[0] = 1.f;
		BottomRight.TexCoords[1] = 1.f;

		for (int32 Index = 0; Index < Verts.Num(); ++Index)
		{
			Verts[Index].MaterialTexCoords[0] = Verts[Index].TexCoords[0];
			Verts[Index].MaterialTexCoords[1] = Verts[Index].TexCoords[1];
		}

		TArray<SlateIndex> Indices = {0, 2, 3, 0, 3, 1};

		FVector2D Offset = MediaViewerSettings.Offset;
		Offset.X = FMath::Fmod(Offset.X, BackgroundTextureBrush.ImageSize.X);
		Offset.Y = FMath::Fmod(Offset.Y, BackgroundTextureBrush.ImageSize.Y);

		FVector2D Start = FVector2D::ZeroVector;
		Start.X = (Offset.X > 0) ? (-BackgroundTextureBrush.ImageSize.X + Offset.X) : Offset.X;
		Start.Y = (Offset.Y > 0) ? (-BackgroundTextureBrush.ImageSize.Y + Offset.Y) : Offset.Y;

		for (float X = Start.X; X < ViewerSize.X; X += ImageSize.X)
		{
			for (float Y = Start.Y; Y < ViewerSize.Y; Y += ImageSize.Y)
			{
				TopLeft.Position.X = BaseVert.Position.X + X;
				TopLeft.Position.Y = BaseVert.Position.Y + Y;

				TopRight.Position.X = BaseVert.Position.X + X + ImageSize.X;
				TopRight.Position.Y = BaseVert.Position.Y + Y;

				BottomLeft.Position.X = BaseVert.Position.X + X;
				BottomLeft.Position.Y = BaseVert.Position.Y + Y + ImageSize.Y;

				BottomRight.Position.X = BaseVert.Position.X + X + ImageSize.X;
				BottomRight.Position.Y = BaseVert.Position.Y + Y + ImageSize.Y;

				FSlateDrawElement::MakeCustomVerts(
					InOutDrawElements,
					InLayerId,
					BackgroundTextureBrush.GetRenderingResource(),
					Verts,
					Indices,
					nullptr,
					0,
					0,
					ESlateDrawEffect::NoPixelSnapping
				);
			}
		}

		++InLayerId;
	}

	FFloatRange UVRange = FFloatRange(0.f, 1.f);

	FMediaImageSlatePaintParams PaintParams = {
		InArgs,
		InAllottedGeometry,
		InMyCullingRect,
		InWidgetStyle,
		bInParentEnabled,
		UVRange,
		GetDPIScale(),
		MediaViewerSettings.ABOrientation,
		ViewerSize,
		ViewerPosition,
		1.f,
		InLayerId,
		InOutDrawElements
	};

	const float ABSplitterLocation = MediaViewerSettings.ABSplitterLocation / 100.f;
	const float SecondImageOpacity = MediaViewerSettings.SecondImageOpacity / 100.f;

	switch (ActiveView)
	{
		case EMediaImageViewerActivePosition::Single:
			GetImageViewer(EMediaImageViewerPosition::First)->Paint(PaintParams);
			break;

		case EMediaImageViewerActivePosition::Both:
			UVRange.SetLowerBound(0.f);
			UVRange.SetUpperBound(1.f);
			GetImageViewer(EMediaImageViewerPosition::First)->Paint(PaintParams);

			if (!FMath::IsNearlyZero(ABSplitterLocation - 1.f))
			{
				UVRange.SetLowerBound(ABSplitterLocation);
				UVRange.SetUpperBound(1.f);
				PaintParams.ImageOpacity = SecondImageOpacity;
				GetImageViewer(EMediaImageViewerPosition::Second)->Paint(PaintParams);
			}
			break;
	}

	const int32 NewLayerId = SCompoundWidget::OnPaint(InArgs, InAllottedGeometry, InMyCullingRect, InOutDrawElements, PaintParams.LayerId, InWidgetStyle, bInParentEnabled);

	// Doing this after rendering or it creates flicker.
	if (ContentSlot)
	{
		// InAllowedGeometry.GetAbsolutePosition() is relative to the parent.
		// GetTickSpaceGeometry().GetAbsolutePosition() is in desktop space.
		ViewerPosition = (ContentSlot->GetWidget()->GetTickSpaceGeometry().GetAbsolutePosition() - GetTickSpaceGeometry().GetAbsolutePosition()) / InAllottedGeometry.Scale;
		ViewerSize = ContentSlot->GetWidget()->GetTickSpaceGeometry().GetAbsoluteSize() / InAllottedGeometry.Scale;
	}

	return NewLayerId;
}

FReply SMediaViewer::OnKeyDown(const FGeometry& InMyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return SCompoundWidget::OnKeyDown(InMyGeometry, InKeyEvent);
}

void SMediaViewer::AddReferencedObjects(FReferenceCollector& InCollector)
{
	InCollector.AddPropertyReferencesWithStructARO(
		FMediaViewerSettings::StaticStruct(),
		&MediaViewerSettings
	);
}

FString SMediaViewer::GetReferencerName() const
{
	static const FString ReferencerName = TEXT("SMediaViewer");
	return ReferencerName;
}

void SMediaViewer::CreateDelegates()
{
	Delegates = MakeShared<FMediaViewerDelegates>();

	// Global viewer delegates
	Delegates->SetSingleView.BindSP(this, &SMediaViewer::SetSingleView);
	Delegates->SetABView.BindSP(this, &SMediaViewer::SetABView);
	Delegates->GetABOrientation.BindSPLambda(this, [this] { return MediaViewerSettings.ABOrientation; });
	Delegates->SetABOrientation.BindSP(this, &SMediaViewer::SetABOrientation);
	Delegates->GetActiveView.BindSPLambda(this, [this] { return ActiveView; });
	Delegates->GetSettings.BindSPLambda(this, [this]{ return &MediaViewerSettings; });
	Delegates->AreTransformsLocked.BindSPLambda(this, [this] { return MediaViewerSettings.bAreTransformsLocked; });
	Delegates->ToggleLockedTransform.BindSPLambda(this, [this] { MediaViewerSettings.bAreTransformsLocked = !MediaViewerSettings.bAreTransformsLocked; });
	Delegates->GetViewerSize.BindSPLambda(this, [this] { return FIntPoint(ViewerSize.X, ViewerSize.Y); });
	Delegates->GetViewerPosition.BindSPLambda(this, [this] { return FIntPoint(ViewerPosition.X, ViewerPosition.Y); });
	Delegates->SwapAB.BindSP(this, &SMediaViewer::SwapABImageViewers);
	Delegates->GetCursorLocation.BindSP(this, &SMediaViewer::GetLocalCursorPosition);
	Delegates->AddOffsetToAll.BindSP(this, &SMediaViewer::AddOffsetToAll);
	Delegates->AddRotationToAll.BindSP(this, &SMediaViewer::AddRotationToAll);
	Delegates->MultiplyScaleToAll.BindSP(this, &SMediaViewer::MultiplyScaleToAll);
	Delegates->MultiplyScaleAroundCursorToAll.BindSP(this, &SMediaViewer::MultiplyScaleAroundCursorToAll);
	Delegates->SetTransformToAll.BindSP(this, &SMediaViewer::SetTransformToAll);
	Delegates->ResetTransformToAll.BindSP(this, &SMediaViewer::ResetTransformToAll);
	Delegates->MirrorAll.BindSP(this, &SMediaViewer::MirrorAll);
	Delegates->GetSecondImageViewerOpacity.BindSPLambda(this, [this] { return MediaViewerSettings.SecondImageOpacity; });
	Delegates->SetSecondImageViewerOpacity.BindSPLambda(this, [this](float InValue) { MediaViewerSettings.SecondImageOpacity = InValue; });
	Delegates->GetABSplitterLocation.BindSPLambda(this, [this] { return MediaViewerSettings.ABSplitterLocation; });
	Delegates->SetABSplitterLocation.BindSPLambda(this, [this](float InValue) { OnABResized(InValue * 0.01f); });
	Delegates->GetLibrary.BindSPLambda(this, [this] { return Library->GetLibrary(); });
	Delegates->RefreshView.BindSPLambda(this, [this] { InvalidateView(); });
	Delegates->GetCommandList.BindSPLambda(this, [this] { return CommandList; });
	Delegates->IsOverViewer.BindSPLambda(this, [this] { return IsHovered(); });

	// Per panel delegates
	Delegates->GetImageViewer.BindSP(this, &SMediaViewer::GetImageViewer);
	Delegates->SetImageViewer.BindSP(this, &SMediaViewer::SetImageViewer);
	Delegates->ClearImageViewer.BindSP(this, &SMediaViewer::ClearImageViewer);
	Delegates->GetPixelCoordinates.BindSP(this, &SMediaViewer::GetPixelCoordinates);
	Delegates->IsOverImage.BindSP(this, &SMediaViewer::IsOverImage);
	Delegates->CopyTransformToAll.BindSP(this, &SMediaViewer::CopyTransformToAll);
	Delegates->GetCommandListForPosition.BindSP(this, &SMediaViewer::GetOverlayCommandList);
}

TSharedRef<SMediaImageViewerOverlay> SMediaViewer::CreateOverlay(EMediaImageViewerPosition InPosition, bool bInComparisonView)
{
	bool bScaleToFitImage = false;
	const int32 Index = static_cast<int32>(InPosition);

	if (ScaleToFit[Index])
	{
		bScaleToFitImage = true;
		ScaleToFit[Index] = false;
	}

	return SNew(SMediaImageViewerOverlay, InPosition, Delegates.ToSharedRef())
		.bComparisonView(bInComparisonView)
		.bScaleToFit(bScaleToFitImage);
}

TSharedRef<SWidget> SMediaViewer::CreateToolbar()
{
	return SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SColorBlock)
			.Color(FStyleColors::Panel.GetSpecifiedColor())
		]
		+ SOverlay::Slot()
		[
			SNew(SMediaViewerToolbar, Delegates.ToSharedRef())
		];
}

void SMediaViewer::CreateView()
{
	ActiveView = RequestedView;

	switch (ActiveView)
	{
		case EMediaImageViewerActivePosition::Single:
			CreateSingleView(EMediaImageViewerPosition::First);
			break;

		case EMediaImageViewerActivePosition::Both:
			CreateABView();
			break;
	}

	if (MediaViewerArgs.bShowToolbar)
	{
		ToolbarSlot->AttachWidget(CreateToolbar());
	}

	bInvalidated = false;
}

void SMediaViewer::CreateSingleView(EMediaImageViewerPosition InPosition)
{
	TSharedRef<SMediaImageViewerOverlay> Overlay = CreateOverlay(InPosition, /* Show Panel Name */ false);

	// Attach the single overlay
	ContentSlot->AttachWidget(Overlay);

	Overlay->PostConstruct();
}

void SMediaViewer::CreateABView()
{
	MediaViewerSettings.ABSplitterLocation = 50.f;

	TSharedRef<SMediaImageViewerOverlay> First = CreateOverlay(EMediaImageViewerPosition::First, /* Show Panel Name */ true);
	TSharedRef<SMediaImageViewerOverlay> Second = CreateOverlay(EMediaImageViewerPosition::Second, /* Show Panel Name */ true);

	ContentSlot->AttachWidget(
		SNew(SSplitter)
		.Orientation(MediaViewerSettings.ABOrientation)
		.PhysicalSplitterHandleSize(3.f)
		.HitDetectionSplitterHandleSize(3.f)

		+ SSplitter::Slot()
		.Value(0.5f)
		.MinSize(10.f)
		.Resizable(true)
		.SizeRule(SSplitter::FractionOfParent)
		.OnSlotResized(this, &SMediaViewer::OnABResized)
		[
			First
		]

		+ SSplitter::Slot()
		.Value(0.5f)
		.MinSize(10.f)
		.Resizable(true)
		.SizeRule(SSplitter::FractionOfParent)
		[
			Second
		]
	);

	First->PostConstruct();
	Second->PostConstruct();
}

TSharedPtr<SMediaImageViewerOverlay> SMediaViewer::GetOverlay(EMediaImageViewerPosition InPosition) const
{
	TSharedRef<SWidget> ContentWidget = ContentSlot->GetWidget();

	switch (ActiveView)
	{
		case EMediaImageViewerActivePosition::Single:
			if (InPosition == EMediaImageViewerPosition::First)
			{
				return StaticCastSharedRef<SMediaImageViewerOverlay>(ContentWidget);
			}
			break;

		case EMediaImageViewerActivePosition::Both:
		{
			TSharedRef<SSplitter> Splitter = StaticCastSharedRef<SSplitter>(ContentSlot->GetWidget());

			if (Splitter->NumSlots() == static_cast<int32>(EMediaImageViewerPosition::COUNT))
			{
				return StaticCastSharedRef<SMediaImageViewerOverlay>(
					Splitter->SlotAt(static_cast<int32>(InPosition)).GetWidget()
				);
			}
		}
	}

	return nullptr;
}

float SMediaViewer::GetDPIScale() const
{
	if (const TSharedPtr<SWindow> TopLevelWindow = FSlateApplication::Get().FindWidgetWindow(SharedThis(this)))
	{
		if (TopLevelWindow.IsValid())
		{
			return TopLevelWindow->GetDPIScaleFactor();
		}
	}

	return 1.0f;
}

void SMediaViewer::OnABResized(float InSize)
{
	if (ActiveView != EMediaImageViewerActivePosition::Single)
	{
		if (TSharedPtr<SMediaImageViewerOverlay> FirstOverlay = GetOverlay(EMediaImageViewerPosition::First))
		{
			StaticCastSharedRef<SSplitter>(ContentSlot->GetWidget())->SlotAt(0).SetSizeValue(InSize);
		}

		if (TSharedPtr<SMediaImageViewerOverlay> FirstOverlay = GetOverlay(EMediaImageViewerPosition::Second))
		{
			StaticCastSharedRef<SSplitter>(ContentSlot->GetWidget())->SlotAt(1).SetSizeValue(1.f - InSize);
		}
	}

	MediaViewerSettings.ABSplitterLocation = InSize * 100.f;
}

EMediaImageViewerPosition SMediaViewer::GetHoveredImageViewer() const
{
	const float ABSplitterLocation = MediaViewerSettings.ABSplitterLocation / 100.f;

	switch (MediaViewerSettings.ABOrientation)
	{
		default:
		case Orient_Horizontal:
			return (ABSplitterLocation * GetTickSpaceGeometry().GetLocalSize().X) < CursorLocalPosition.X
				? EMediaImageViewerPosition::First
				: EMediaImageViewerPosition::Second;

		case Orient_Vertical:
			return (ABSplitterLocation * GetTickSpaceGeometry().GetLocalSize().Y) < CursorLocalPosition.Y
				? EMediaImageViewerPosition::First
				: EMediaImageViewerPosition::Second;
	}
}

FVector2D SMediaViewer::GetLocalCursorPosition() const
{
	return CursorLocalPosition;
}

TSharedRef<FMediaViewerDelegates> SMediaViewer::GetDelegates() const
{
	return Delegates.ToSharedRef();
}

FMediaViewerState SMediaViewer::GetState() const
{
	FMediaViewerState State;
	State.ViewerSettings = MediaViewerSettings;
	State.ActiveView = ActiveView;

	for (int32 Index = 0; Index < static_cast<int32>(EMediaImageViewerPosition::COUNT); ++Index)
	{
		bool bAdded = false;

		if (ImageViewers[Index].IsValid())
		{
			if (TSharedPtr<FMediaViewerLibraryItem> LibraryItem = ImageViewers[Index]->CreateLibraryItem())
			{
				State.Images.Add({
					LibraryItem->GetItemType(),
					LibraryItem->GetStringValue(),
					ImageViewers[Index]->GetPanelSettings(),
					ImageViewers[Index]->GetPaintSettings()
					});
				bAdded = true;
			}
		}

		if (!bAdded)
		{
			State.Images.Add({});
		}
	}

	return State;
}

void SMediaViewer::LoadState(const FMediaViewerState& InState)
{
	TArray<TSharedPtr<FMediaViewerLibraryItem>> Items;
	Items.Reserve(InState.Images.Num());
	bool bHasValidItem = false;

	for (int32 Index = 0; Index < InState.Images.Num(); ++Index)
	{
		TSharedPtr<FMediaViewerLibraryItem> Item = Library->GetLibrary()->FindItemByValue(
			InState.Images[Index].ImageType,
			InState.Images[Index].StringValue
		);

		Items.Add(Item);

		if (Item.IsValid())
		{
			bHasValidItem = true;
		}
	}

	if (!bHasValidItem)
	{
		return;
	}

	MediaViewerSettings = InState.ViewerSettings;
	RequestedView = InState.ActiveView;

	for (int32 Index = 0; Index < InState.Images.Num(); ++Index)
	{
		if (Items[Index].IsValid())
		{
			if (TSharedPtr<FMediaImageViewer> Viewer = Items[Index]->CreateImageViewer())
			{
				ImageViewers[Index] = Viewer.ToSharedRef();
				Viewer->GetPanelSettings() = InState.Images[Index].PanelSettings;
				Viewer->GetPaintSettings() = InState.Images[Index].PaintSettings;
				Viewer->InitBackgroundTexture();
			}
		}
	}

	InvalidateView();
}

void SMediaViewer::SaveBookmark(int32 InIndex, bool bInSaveConfig)
{
	UMediaViewerLibraryIni& Ini = UMediaViewerLibraryIni::Get();
	Ini.SetBookmark(InIndex, GetState());

	if (bInSaveConfig)
	{
		Ini.SaveConfig();
	}
}

void SMediaViewer::LoadBookmark(int32 InIndex)
{
	TConstArrayView<FMediaViewerState> Bookmarks = UMediaViewerLibraryIni::Get().GetBookmarks();

	if (!Bookmarks.IsValidIndex(InIndex))
	{
		return;
	}

	LoadState(Bookmarks[InIndex]);
}

void SMediaViewer::SaveLastOpenedState(bool bInSaveConfig)
{
	UMediaViewerLibraryIni& Ini = UMediaViewerLibraryIni::Get();
	Ini.SetLastOpenedState(GetState());

	if (bInSaveConfig)
	{
		Ini.SaveConfig();
	}
}

void SMediaViewer::InvalidateView()
{
	bInvalidated = true;
}

bool SMediaViewer::IsImageViewerNull(EMediaImageViewerPosition InPosition) const
{
	return !ImageViewers[static_cast<int32>(InPosition)].IsValid()
		|| ImageViewers[static_cast<int32>(InPosition)]->GetInfo().Id == FNullImageViewer::GetNullImageViewer()->GetInfo().Id;
}

void SMediaViewer::AddOffsetToAll(const FVector& InOffset)
{
	for (const TSharedPtr<FMediaImageViewer>& ImageViewer : ImageViewers)
	{
		if (ImageViewer.IsValid())
		{
			ImageViewer->GetPaintSettings().Offset += InOffset;
		}
	}
}

void SMediaViewer::AddRotationToAll(const FRotator& InRotation)
{
	for (const TSharedPtr<FMediaImageViewer>& ImageViewer : ImageViewers)
	{
		if (ImageViewer.IsValid())
		{
			ImageViewer->GetPaintSettings().Rotation += InRotation;
		}
	}
}

void SMediaViewer::MultiplyScaleToAll(float InMultiple)
{
	for (const TSharedPtr<FMediaImageViewer>& ImageViewer : ImageViewers)
	{
		if (ImageViewer.IsValid())
		{
			ImageViewer->GetPaintSettings().Scale = FMath::Clamp(ImageViewer->GetPaintSettings().Scale * InMultiple, SMediaViewer::MinScale, SMediaViewer::MaxScale);
		}
	}
}

void SMediaViewer::MultiplyScaleAroundCursorToAll(float InMultiple)
{
	for (int32 Index = 0; Index < static_cast<int32>(EMediaImageViewerPosition::COUNT); ++Index)
	{
		if (TSharedPtr<SMediaImageViewerOverlay> Overlay = GetOverlay(static_cast<EMediaImageViewerPosition>(Index)))
		{
			Overlay->MultiplyScaleAroundCursor(InMultiple);
		}
	}
}

void SMediaViewer::SetTransformToAll(const FVector& InOffset, const FRotator& InRotation, float InScale)
{
	for (const TSharedPtr<FMediaImageViewer>& ImageViewer : ImageViewers)
	{
		if (ImageViewer.IsValid())
		{
			ImageViewer->GetPaintSettings().Offset = InOffset;
			ImageViewer->GetPaintSettings().Rotation = InRotation;
			ImageViewer->GetPaintSettings().Scale = InScale;
		}
	}
}

void SMediaViewer::ResetTransformToAll()
{
	SetTransformToAll(FVector::ZeroVector, FRotator::ZeroRotator, 1.f);
}

FIntPoint SMediaViewer::GetPixelCoordinates(EMediaImageViewerPosition InPosition) const
{
	if (TSharedPtr<SMediaImageViewerOverlay> Overlay = GetOverlay(InPosition))
	{
		return Overlay->GetImageViewerPixelCoordinates();
	}

	return FIntPoint(-1, -1);
}

bool SMediaViewer::IsOverImage(EMediaImageViewerPosition InPosition) const
{
	if (TSharedPtr<SMediaImageViewerOverlay> Overlay = GetOverlay(InPosition))
	{
		return Overlay->IsCursorOverImageViewer();
	}

	return false;
}

void SMediaViewer::CopyTransformToAll(EMediaImageViewerPosition InPosition) const
{
	const int32 SourceIndex = static_cast<int32>(InPosition);
	const FMediaImagePaintSettings& SourcePaintSettings = ImageViewers[SourceIndex]->GetPaintSettings();

	for (int32 Index = 0; Index < static_cast<int32>(EMediaImageViewerPosition::COUNT); ++Index)
	{
		if (Index != SourceIndex)
		{
			FMediaImagePaintSettings& PaintSettings = ImageViewers[Index]->GetPaintSettings();
			PaintSettings.Offset = SourcePaintSettings.Offset;
			PaintSettings.Rotation = SourcePaintSettings.Rotation;
			PaintSettings.Scale = SourcePaintSettings.Scale;
		}
	}
}

void SMediaViewer::MirrorAll(EMediaImageMirrorPlane InPlane, bool bInSet)
{
	for (int32 Index = 0; Index < static_cast<int32>(EMediaImageViewerPosition::COUNT); ++Index)
	{
		ImageViewers[Index]->SetMirrored(InPlane, bInSet);
	}
}

TSharedPtr<FUICommandList> SMediaViewer::GetOverlayCommandList(EMediaImageViewerPosition InPosition) const
{
	if (TSharedPtr<SMediaImageViewerOverlay> Overlay = GetOverlay(InPosition))
	{
		return Overlay->GetCommandList();
	}

	return nullptr;
}

float SMediaViewer::GetSecondImageOpacity() const
{
	return MediaViewerSettings.SecondImageOpacity;
}

void SMediaViewer::AdjustSecondImageOpacity(float InAdjustment)
{
	SetSecondImageOpacity(GetSecondImageOpacity() + InAdjustment);
}

void SMediaViewer::SetSecondImageOpacity(float InOpacity)
{
	MediaViewerSettings.SecondImageOpacity = InOpacity;
}

ECheckBoxState SMediaViewer::AreOverlaysEnabled() const
{
	for (int32 Index = 0; Index < static_cast<int32>(EMediaImageViewerPosition::COUNT); ++Index)
	{
		if (TSharedPtr<SMediaImageViewerOverlay> Overlay = GetOverlay(static_cast<EMediaImageViewerPosition>(Index)))
		{
			if (Overlay->IsOverlayEnabled())
			{
				return ECheckBoxState::Checked;
			}
		}
	}

	return ECheckBoxState::Unchecked;
}

void SMediaViewer::ToggleOverlays()
{
	for (int32 Index = 0; Index < static_cast<int32>(EMediaImageViewerPosition::COUNT); ++Index)
	{
		if (TSharedPtr<SMediaImageViewerOverlay> Overlay = GetOverlay(static_cast<EMediaImageViewerPosition>(Index)))
		{
			Overlay->ToggleOverlay();
		}
	}
}

ECheckBoxState SMediaViewer::AreLockedTransformEnabled() const
{
	return MediaViewerSettings.bAreTransformsLocked
		? ECheckBoxState::Checked
		: ECheckBoxState::Unchecked;
}

void SMediaViewer::ToggleLockedTransform()
{
	MediaViewerSettings.bAreTransformsLocked = !MediaViewerSettings.bAreTransformsLocked;
}

void SMediaViewer::CheckLastOpenedState()
{
	const FGuid& NullImageId = FNullImageViewer::GetNullImageViewer()->GetInfo().Id;

	for (int32 Index = 0; Index < static_cast<int32>(EMediaImageViewerPosition::COUNT); ++Index)
	{
		if (ImageViewers[Index].IsValid() && ImageViewers[Index]->GetInfo().Id != NullImageId)
		{
			return;
		}
	}

	const EAppReturnType::Type Result = FMessageDialog::Open(
		EAppMsgType::YesNo,
		LOCTEXT("LoadOldState", "Attempt to open previous images?")
	);

	if (Result == EAppReturnType::Yes)
	{
		LoadLastOpenedState();
	}
}

void SMediaViewer::LoadLastOpenedState()
{
	TOptional<FMediaViewerState> LastOpenedState = UMediaViewerLibraryIni::Get().GetLastOpenedState();

	if (!LastOpenedState.IsSet())
	{
		return;
	}

	LoadState(LastOpenedState.GetValue());
}

bool SMediaViewer::CanTakeSnapshot() const
{
	switch (ActiveView)
	{
		case EMediaImageViewerActivePosition::Single:
			return !IsImageViewerNull(EMediaImageViewerPosition::First);

		case EMediaImageViewerActivePosition::Both:
			return !IsImageViewerNull(EMediaImageViewerPosition::First)
				|| !IsImageViewerNull(EMediaImageViewerPosition::Second);

		default:
			return false;
	}
}

void SMediaViewer::TakeSnapshot()
{
	if (!ContentSlot)
	{
		return;
	}

	if (IsImageViewerNull(EMediaImageViewerPosition::First)
		&& IsImageViewerNull(EMediaImageViewerPosition::Second))
	{
		return;
	}

	if (FMath::IsNearlyZero(ViewerSize.X) || FMath::IsNearlyZero(ViewerSize.Y))
	{
		return;
	}

	TStrongObjectPtr<UTextureRenderTarget2D> RenderTarget = TStrongObjectPtr<UTextureRenderTarget2D>(FMediaViewerUtils::CreateRenderTarget(
		FIntPoint(
			FMath::RoundToInt(ViewerSize.X),
			FMath::RoundToInt(ViewerSize.Y)
		),
		/* Transparent */ true
	));

	if (!RenderTarget)
	{
		return;
	}

	// Choose asset location
	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	const FContentBrowserItemPath CurrentPath = ContentBrowser.GetCurrentPath();
	const FString PathStr = CurrentPath.HasInternalPath() ? CurrentPath.GetInternalPathString() : "/Game";

	FSaveAssetDialogConfig SaveAssetDialogConfig;
	SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveSnapshotAs", "Save Snapshot As");
	SaveAssetDialogConfig.DefaultPath = PathStr;
	SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Disallow;
	SaveAssetDialogConfig.DefaultAssetName = LOCTEXT("MediaViewerSnapshot", "MediaViewerSnapshot").ToString();

	for (int32 Index = 0; Index < static_cast<int32>(EMediaImageViewerPosition::COUNT); ++Index)
	{
		if (TSharedPtr<FMediaImageViewer> ImageViewer = GetImageViewer(static_cast<EMediaImageViewerPosition>(Index)))
		{
			if (!ImageViewer->IsValid())
			{
				continue;
			}

			SaveAssetDialogConfig.DefaultAssetName += TEXT("_");

			FString DisplayName = ImageViewer->GetInfo().DisplayName.ToString();

			if (FPaths::FileExists(DisplayName))
			{
				DisplayName = FPaths::GetCleanFilename(DisplayName);
			}

			SaveAssetDialogConfig.DefaultAssetName += UPackageTools::SanitizePackageName(DisplayName);
		}
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

	if (SaveObjectPath.IsEmpty())
	{
		return;
	}

	FTextureRenderTargetResource* RenderTargetResource = RenderTarget->GameThread_GetRenderTargetResource();

	UCanvas* Canvas = NewObject<UCanvas>(GetTransientPackage());

	FCanvas RenderCanvas(
		RenderTargetResource,
		nullptr,
		FGameTime::CreateUndilated(0, 0),
		GEngine->GetDefaultWorldFeatureLevel()
	);

	Canvas->Init(RenderTarget->SizeX, RenderTarget->SizeY, nullptr, &RenderCanvas);

	{
		RHI_BREADCRUMB_EVENT_GAMETHREAD_F("CreateMediaViewerSnapshot", "CreateMediaViewerSnapshot: %s", RenderTarget->GetFName());

		ENQUEUE_RENDER_COMMAND(FlushDeferredResourceUpdateCommand)(
			[RenderTargetResource](FRHICommandListImmediate& RHICmdList)
			{
				RenderTargetResource->FlushDeferredResourceUpdate(RHICmdList);
			});

		DrawImageViewers(Canvas);

		RenderCanvas.Flush_GameThread();
		Canvas->Canvas = nullptr;

		//UpdateResourceImmediate must be called here to ensure mips are generated.
		RenderTarget->UpdateResourceImmediate(false);

		ENQUEUE_RENDER_COMMAND(ResetSceneTextureExtentHistory)(
			[RenderTargetResource](FRHICommandListImmediate& RHICmdList)
			{
				RenderTargetResource->ResetSceneTextureExtentsHistory();
			});
	}

	const FString PackageName = FPaths::GetBaseFilename(*SaveObjectPath, false);
	const FString AssetName = FPaths::GetBaseFilename(*SaveObjectPath, true);

	UTexture2D* Texture = RenderTarget->ConstructTexture2D(GetTransientPackage(), AssetName, RF_Public | RF_Standalone);

	if (!Texture)
	{
		return;
	}

	RenderTarget->ReleaseResource();

	// Create package and save texture.
	UPackage* Package = CreatePackage(*PackageName);

	if (!Package)
	{
		return;
	}

	Texture->Rename(*AssetName, Package, REN_DontCreateRedirectors | REN_DoNotDirty | REN_AllowPackageLinkerMismatch | REN_NonTransactional);

	FAssetRegistryModule::AssetCreated(Package);

	// Add to snapshots group
	TSharedRef<FMediaViewerLibraryItem> SnapshotItem = MakeShared<FTexture2DImageViewer::FItem>(
		FText::FromString(AssetName),
		LOCTEXT("SnapshotTooltip", "Snapshot from Media Viewer"),
		/* Transient */ false,
		TNotNull<UTexture2D*>(Texture)
	);

	TSharedRef<IMediaViewerLibrary> LibraryActual = GetLibrary();
	LibraryActual->AddItemToGroup(SnapshotItem, LibraryActual->GetSnapshotsGroupId());

	// Make a toast!
	const FString HyperLinkText = FPaths::ConvertRelativePathToFull(SaveObjectPath);

	FNotificationInfo Info(LOCTEXT("SnapshotCreated", "Snapshot created."));

	Info.Hyperlink = FSimpleDelegate::CreateStatic([](FString SourceFilePath)
		{
			FPlatformProcess::ExploreFolder(*(FPaths::GetPath(SourceFilePath)));
		}, HyperLinkText);

	Info.HyperlinkText = FText::FromString(HyperLinkText);
	Info.ExpireDuration = 3.0f;

	FSlateNotificationManager::Get().AddNotification(Info);
}

void SMediaViewer::DrawImageViewers(UCanvas* InCanvas) const
{
	if (UTexture* BackgroundTexture = MediaViewerSettings.Texture.LoadSynchronous())
	{
		const float BackgroundWidth = BackgroundTexture->GetSurfaceWidth();
		const float BackgroundHeight = BackgroundTexture->GetSurfaceHeight();

		InCanvas->DrawTile(
			BackgroundTexture,
			/* X */ 0,
			/* Y */ 0,
			/* XL */ BackgroundWidth,
			/* YL */ BackgroundHeight,
			/* U */ 0,
			/* V */ 0,
			/* UL */ static_cast<float>(InCanvas->SizeX) / BackgroundWidth,
			/* VL */ static_cast<float>(InCanvas->SizeY) / BackgroundHeight
		);
	}

	FFloatRange UVRange = FFloatRange(0.f, 1.f);

	FMediaImageCanvasPaintParams PaintParams = {
		InCanvas,
		UVRange,
		MediaViewerSettings.ABOrientation,
		1.f
	};

	const float ABSplitterLocation = MediaViewerSettings.ABSplitterLocation / 100.f;
	const float SecondImageOpacity = MediaViewerSettings.SecondImageOpacity / 100.f;

	switch (ActiveView)
	{
		case EMediaImageViewerActivePosition::Single:
			GetImageViewer(EMediaImageViewerPosition::First)->Paint(PaintParams);
			break;

		case EMediaImageViewerActivePosition::Both:
			UVRange.SetLowerBound(0.f);
			UVRange.SetUpperBound(1.f);
			GetImageViewer(EMediaImageViewerPosition::First)->Paint(PaintParams);

			if (!FMath::IsNearlyZero(ABSplitterLocation - 1.f))
			{
				UVRange.SetLowerBound(ABSplitterLocation);
				UVRange.SetUpperBound(1.f);
				PaintParams.ImageOpacity = SecondImageOpacity;
				GetImageViewer(EMediaImageViewerPosition::Second)->Paint(PaintParams);
			}
			break;
	}
}

void SMediaViewer::OpenNewTab()
{
	IMediaViewerModule::Get().OpenStandaloneTab();
}

bool SMediaViewer::CanOpenBookmark(int32 InIndex) const
{
	TConstArrayView<FMediaViewerState> Bookmarks = UMediaViewerLibraryIni::Get().GetBookmarks();

	if (!Bookmarks.IsValidIndex(InIndex))
	{
		return false;
	}

	for (const FMediaViewerImageState& Image : Bookmarks[InIndex].Images)
	{
		if (!Image.StringValue.IsEmpty())
		{
			return true;
		}
	}

	return false;
}

void SMediaViewer::OpenBookmark(int32 InIndex)
{
	LoadBookmark(InIndex);

	UE_LOGF(LogMediaViewer, Log, "Bookmark %i loaded.", InIndex + 1);

	const FText MessageFormat = LOCTEXT("BookmarkIndexLoaded", "[Media Viewer] Bookmark {0} loaded.");

	FNotificationInfo NotificationInfo(FText::Format(MessageFormat, FText::AsNumber(InIndex + 1)));
	NotificationInfo.ExpireDuration = 3.f;
	NotificationInfo.bFireAndForget = true;

	FSlateNotificationManager::Get().AddNotification(NotificationInfo);
}

bool SMediaViewer::CanSaveBookmark() const
{
	return !IsImageViewerNull(EMediaImageViewerPosition::First) || !IsImageViewerNull(EMediaImageViewerPosition::Second);
}

void SMediaViewer::SaveBookmark(int32 InIndex)
{
	SaveBookmark(InIndex, /* Save config */ true);

	UE_LOGF(LogMediaViewer, Log, "Bookmark %i saved.", InIndex + 1);

	const FText MessageFormat = LOCTEXT("BookmarkIndexSaved", "[Media Viewer] Bookmark {0} saved.");

	FNotificationInfo NotificationInfo(FText::Format(MessageFormat, FText::AsNumber(InIndex + 1)));
	NotificationInfo.ExpireDuration = 3.f;
	NotificationInfo.bFireAndForget = true;

	FSlateNotificationManager::Get().AddNotification(NotificationInfo);
}

void SMediaViewer::OnOwnerCleanup(UObject* InOuter)
{
	const FGuid& NullImageId = FNullImageViewer::GetNullImageViewer()->GetInfo().Id;

	for (int32 Index = 0; Index < static_cast<int32>(EMediaImageViewerPosition::COUNT); ++Index)
	{
		if (ImageViewers[Index].IsValid() && ImageViewers[Index]->GetInfo().Id != NullImageId)
		{
			if (ImageViewers[Index]->OnOwnerCleanup(InOuter))
			{
				ClearImageViewer(static_cast<EMediaImageViewerPosition>(Index));
			}
		}
	}
}

void SMediaViewer::OnWorldCleanUp(UWorld* InWorld, bool bInSessionEnded, bool bInCleanupResources)
{
	if (bInCleanupResources)
	{
		OnOwnerCleanup(InWorld);
	}
}

void SMediaViewer::OnPackageDeleted(UPackage* InPackage)
{
	OnOwnerCleanup(InPackage);
}

void SMediaViewer::OnAssetsDeleted(const TArray<UObject*>& InAssets)
{
	for (UObject* Asset : InAssets)
	{
		OnOwnerCleanup(Asset);
	}
}

void SMediaViewer::Mirror(EMediaImageMirrorPlane InPlane)
{
	const FGuid& NullImageId = FNullImageViewer::GetNullImageViewer()->GetInfo().Id;

	for (int32 Index = 0; Index < static_cast<int32>(EMediaImageViewerPosition::COUNT); ++Index)
	{
		if (TSharedPtr<SMediaImageViewerOverlay> Overlay = GetOverlay(static_cast<EMediaImageViewerPosition>(Index)))
		{
			Overlay->Try_Mirror(InPlane);
			break;
		}
	}
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
