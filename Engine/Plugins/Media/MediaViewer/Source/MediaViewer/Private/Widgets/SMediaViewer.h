// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/GCObject.h"
#include "Widgets/SCompoundWidget.h"

#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "MediaViewer.h"
#include "Misc/Optional.h"
#include "Widgets/MediaViewerSettings.h"
#include "Widgets/SBoxPanel.h"

#include "SMediaViewer.generated.h"

class FUICommandList;
class SViewportToolBar;
class UCanvas;
class UPackage;
class UWorld;
enum class EMediaImageMirrorPlane : uint8;
struct FMediaViewerState;

namespace UE::MediaViewer
{
	class FMediaImageStatusBarExtender;
	class FMediaImageViewer;
	class IMediaViewerLibrary;
	class SMediaViewerTab;
	struct FMediaViewerDelegates;

	namespace Private
	{
		class FMediaImageViewerViewportClient;
		class SMediaImageViewerOverlay;
		class SMediaImageViewerToolbar;
		class SMediaViewerLibraryPrivate;
	}
}

UENUM()
enum class EMediaImageViewerActivePosition : uint8
{
	Single,
	Both
};

namespace UE::MediaViewer::Private
{

class SMediaViewer : public SCompoundWidget, public FGCObject
{
	SLATE_DECLARE_WIDGET(SMediaViewer, SCompoundWidget)

	SLATE_BEGIN_ARGS(SMediaViewer)
		: _ClearColor(FLinearColor::Black)
		{}
		SLATE_ARGUMENT(FLinearColor, ClearColor)
	SLATE_END_ARGS()

public:
	static constexpr float MaxScale = 20.f;
	static constexpr float MinScale = 0.01f;

	SMediaViewer();

	virtual ~SMediaViewer() override;

	void Construct(const FArguments& InArgs, const TSharedRef<SMediaViewerTab>& InTab, const FMediaViewerArgs& InMediaViewerArgs,
		const TSharedRef<FMediaImageViewer>& InImageViewerFirst, const TSharedRef<FMediaImageViewer>& InImageViewerSecond);

	const FMediaViewerArgs& GetArgs() const { return MediaViewerArgs; }

	TSharedRef<IMediaViewerLibrary> GetLibrary() const;

	TSharedPtr<FMediaImageViewer> GetImageViewer(EMediaImageViewerPosition InPosition) const;

	void SetImageViewer(EMediaImageViewerPosition InPosition, const TSharedRef<FMediaImageViewer>& InImageViewer);
	
	void ClearImageViewer(EMediaImageViewerPosition InPosition);
	
	bool IsShowingBothImageViewers() const;

	void SwapABImageViewers();

	void SetSingleView();

	void SetABView();

	/** Sets the orientation of the A/B view. */
	void SetABOrientation(EOrientation InOrientation);

	/** Returns the position that the mouse is hovering based _only_ on the position of the mouse compared to the splitter. */
	EMediaImageViewerPosition GetHoveredImageViewer() const;

	/** Returns the local position of the cursor on the viewer. */
	FVector2D GetLocalCursorPosition() const;

	/** Triggers a recreation of the view on tick. */
	void InvalidateView();

	/** Returns the bound delegates of this viewer. */
	TSharedRef<FMediaViewerDelegates> GetDelegates() const;

	FMediaViewerState GetState() const;

	void LoadState(const FMediaViewerState& InState);

	void SaveBookmark(int32 InIndex, bool bInSaveConfig);

	void LoadBookmark(int32 InIndex);

	void SaveLastOpenedState(bool bInSaveConfig);

	//~ Begin SWidget
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual int32 OnPaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InMyCullingRect, 
		FSlateWindowElementList& InOutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const override;
	virtual FReply OnKeyDown(const FGeometry& InMyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget

	//~ Begin FGCObject
	virtual FString GetReferencerName() const override;
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	//~ End FGCObject

protected:
	static const FSlateColorBrush BackgroundColorBrush;

	/** Owner of this viewer. */
	TWeakPtr<SMediaViewerTab> Tab;
	/** Construction args (unchangeable) */
	FMediaViewerArgs MediaViewerArgs;
	/** Mutable settings */
	FMediaViewerSettings MediaViewerSettings;
	/** Delegates for sub-widgets. Similar to an advanced command list. */
	TSharedPtr<FMediaViewerDelegates> Delegates;
	/** Command list for the entire viewer. */
	TSharedRef<FUICommandList> CommandList;

	/** Library of available image viewers */
	TSharedPtr<SMediaViewerLibraryPrivate> Library;
	/** Main layout of the viewer. */
	TSharedPtr<SVerticalBox> Layout;
	/** Toolbar slot for direct access and replacement. */
	SVerticalBox::FSlot* ToolbarSlot;
	/** Content slot for direct access and replacement. */
	SVerticalBox::FSlot* ContentSlot;

	/** Background brush used to render the tiled background texture. */
	mutable FSlateImageBrush BackgroundTextureBrush;

	/** Viewers currently open in the viewer. May be FNullImageViewer for an invalid viewer. */
	TSharedPtr<FMediaImageViewer> ImageViewers[static_cast<int32>(EMediaImageViewerPosition::COUNT)];
	/** The active view configuration. */
	EMediaImageViewerActivePosition ActiveView;
	/** The requested view configuration. */
	EMediaImageViewerActivePosition RequestedView;
	/** Whether to scale to fit on rebuild view. */
	bool ScaleToFit[static_cast<int32>(EMediaImageViewerPosition::COUNT)];

	/** A check to see if we need to load old states. */
	int32 TickCount = 0;

	FVector2D CursorLocalPosition;

	mutable FVector2D ViewerSize;
	mutable FVector2D ViewerPosition;

	/** Do we need to be recreated next tick? */
	bool bInvalidated;

	/** Create and assign the list of delegates. */
	void CreateDelegates();

	/** Binds commands. */
	void BindCommands();

	/** Create and overlay widget for the given position. */
	TSharedRef<SMediaImageViewerOverlay> CreateOverlay(EMediaImageViewerPosition InPosition, bool bInComparisonView);

	/** Creates the toolbar. */
	TSharedRef<SWidget> CreateToolbar();

	/** Examines the available images and creates an appropriate view. */
	void CreateView();

	/** Creates a view for a single image viewer. */
	void CreateSingleView(EMediaImageViewerPosition InPosition);

	/** Creates a view comparing 2 images. */
	void CreateABView();

	/** Returns the overlay for the given position, if it exists. */
	TSharedPtr<SMediaImageViewerOverlay> GetOverlay(EMediaImageViewerPosition InPosition) const;

	/** Returns the DPI scale of the window. */
	float GetDPIScale() const;

	/** When the splitter is resized, this is triggered to update @see ABSplitterLocation. */
	void OnABResized(float InSize);

	/** Returns trus if the image viewer is null or an FNullImageViewer. */
	bool IsImageViewerNull(EMediaImageViewerPosition InPosition) const;

	/** Adds an offset to all image viewers. */
	void AddOffsetToAll(const FVector& InOffset);

	/** Adds a rotation to all image viewers. */
	void AddRotationToAll(const FRotator& InRotation);

	/** Multiplays the scale of all image viewers by a value. */
	void MultiplyScaleToAll(float InMultiple);

	/** Multiplays the scale of all image viewers by a value keeping the cursor in the same relative position for each image. */
	void MultiplyScaleAroundCursorToAll(float InMultiple);

	/** Sets the transform of all image viewers to the given values. */
	void SetTransformToAll(const FVector& InOffset, const FRotator& InRotation, float InScale);

	/** Resets the transform of all image viewers to 0,0,0; 0,0,0; 1 */
	void ResetTransformToAll();

	/** Copies the transform of the given image viewer to all other image viewers. */
	void CopyTransformToAll(EMediaImageViewerPosition InPosition) const;

	/** Sets the mirror plane flag on all image viewers. */
	void MirrorAll(EMediaImageMirrorPlane InPlane, bool bInSet);

	/**
	 * Returns the coordinates of the given image that the mousee is currently hovering.
	 * May not be inside the image.
	 */
	FIntPoint GetPixelCoordinates(EMediaImageViewerPosition InPosition) const;

	/** Returns if the mouse is currently hovering the given image. Inside the image. */
	bool IsOverImage(EMediaImageViewerPosition InPosition) const;

	/** Returns the command list of the given overlay. */
	TSharedPtr<FUICommandList> GetOverlayCommandList(EMediaImageViewerPosition InPosition) const;

	/** For dealing with the second/B image opacity. */
	float GetSecondImageOpacity() const;
	void AdjustSecondImageOpacity(float InAdjustment);
	void SetSecondImageOpacity(float InOpacity);

	/** For toggling off the status bar and custom overlays of the image viewers. */
	ECheckBoxState AreOverlaysEnabled() const;
	void ToggleOverlays();

	/** For locking the image viewer transforms. */
	ECheckBoxState AreLockedTransformEnabled() const;
	void ToggleLockedTransform();

	void CheckLastOpenedState();

	bool CanTakeSnapshot() const;
	void TakeSnapshot();

	void DrawImageViewers(UCanvas* InCanvas) const;
	
	void OpenNewTab();

	void LoadLastOpenedState();

	bool CanOpenBookmark(int32 InIndex) const;	
	void OpenBookmark(int32 InIndex);
	bool CanSaveBookmark() const;
	void SaveBookmark(int32 InIndex);

	void OnOwnerCleanup(UObject* InOuter);

	void OnWorldCleanUp(UWorld* InWorld, bool bInSessionEnded, bool bInCleanupResources);

	void OnPackageDeleted(UPackage* InPackage);

	void OnAssetsDeleted(const TArray<UObject*>& InAssets);

	/** Forwards the mirror bind to the first valid viewer. */
	void Mirror(EMediaImageMirrorPlane InPlane);
};

} // UE::MediaViewer::Private
