// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaViewerDropTarget.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetSelection.h"
#include "CommonFrameRates.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "ContentBrowserModule.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Editor.h"
#include "Engine/Texture2D.h"
#include "FileMediaSource.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/FileManager.h"
#include "IContentBrowserSingleton.h"
#include "IMediaEventSink.h"
#include "IMediaStreamPlayer.h"
#include "IMediaViewerModule.h"
#include "ImageUtils.h"
#include "ImageViewers/MediaSourceImageViewer.h"
#include "ImageViewers/Texture2DImageViewer.h"
#include "ImgMediaSource.h"
#include "Input/DragAndDrop.h"
#include "Library/MediaViewerLibrary.h"
#include "Library/MediaViewerLibraryItem.h"
#include "MediaPlayer.h"
#include "MediaStream.h"
#include "MediaViewerDelegates.h"
#include "MediaViewerModule.h"
#include "Misc/MessageDialog.h"
#include "SDropTarget.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "TimerManager.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/MediaViewerLibraryItemDragDropOperation.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SFrameRatePicker.h"
#include "Widgets/SMediaViewerDropTargetInternal.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SMediaViewerDropTarget"

namespace UE::MediaViewer::Private
{

FFrameRate FindClosestCommonFrameRate(float InRate)
{
	FFrameRate BestMatch(24, 1);	// ImgMediaSource Default

	if (InRate <= 0.0f)
	{
		return BestMatch;
	}

	float BestDiff = FMath::Abs(BestMatch.AsDecimal() - static_cast<double>(InRate));

	for (const FCommonFrameRateInfo& Info : FCommonFrameRates::GetAll())
	{
		const float Diff = FMath::Abs(Info.FrameRate.AsDecimal() - static_cast<double>(InRate));

		if (Diff < BestDiff)
		{
			BestDiff = Diff;
			BestMatch = Info.FrameRate;
		}
	}

	return BestMatch;
}

bool GetFrameNumberAndStem(const FString& InFilename, const FString& InExtension, int32& OutFrameNumber, FString& OutStem)
{
	// Start scanning from the right, skipping the extension (and its dot).
	int32 Index = InFilename.Len() - 1 - InExtension.Len();

	// Skip the dot before the extension.
	if (Index >= 0 && InFilename[Index] == TEXT('.'))
	{
		--Index;
	}

	// Find the first digit from the right.
	for (; Index >= 0; --Index)
	{
		if (FChar::IsDigit(InFilename[Index]))
		{
			break;
		}
	}

	if (Index < 0)
	{
		return false;
	}

	const int32 LastDigitIndex = Index;

	// Find the start of the digit sequence.
	for (; Index >= 0; --Index)
	{
		if (!FChar::IsDigit(InFilename[Index]))
		{
			break;
		}
	}

	++Index;

	const FString NumberString = InFilename.Mid(Index, LastDigitIndex - Index + 1);

	// Skip leading zeros when counting significant digits; zero-padded sequence
	// filenames (e.g. "frame_00000042.png") are common and shouldn't be rejected.
	// Stop one before the end so a string of all zeros still has one digit to parse.
	int32 FirstNonZeroIndex = 0;
	while (FirstNonZeroIndex < NumberString.Len() - 1 && NumberString[FirstNonZeroIndex] == TEXT('0'))
	{
		++FirstNonZeroIndex;
	}
	const int32 SignificantDigits = NumberString.Len() - FirstNonZeroIndex;

	// Hard-cap significant digits to avoid Atoi64 overflow UB. int32 fits ~10 digits
	// (MAX_int32 = 2147483647), which is far more than any real frame sequence.
	if (SignificantDigits > 10)
	{
		return false;
	}

	const int64 ParsedNumber = FCString::Atoi64(*NumberString);
	if (ParsedNumber < 0 || ParsedNumber > MAX_int32)
	{
		return false;
	}

	OutStem = InFilename.Left(Index);
	OutFrameNumber = static_cast<int32>(ParsedNumber);

	return true;
}

} // namespace UE::MediaViewer::Private

namespace
{

/** Modal dialog that lets the user pick a frame rate for an image sequence. */
class SImageSequenceFrameRateDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SImageSequenceFrameRateDialog) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<SWindow> InWindow, const FFrameRate& InDefaultFrameRate)
	{
		Window = InWindow;
		SelectedFrameRate = InDefaultFrameRate;

		ChildSlot
		[
			SNew(SBorder)
				.Visibility(EVisibility::Visible)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
						.FillHeight(1)
						.VAlign(VAlign_Top)
						[
							SNew(SBorder)
								.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
								.Padding(4.0f)
								[
									SNew(SVerticalBox)

									+ SVerticalBox::Slot()
										.Padding(0.0f, 0.0f, 0.0f, 4.0f)
										[
											SNew(STextBlock)
												.Text(LOCTEXT("SelectFrameRateLabel", "Select a frame rate for this image sequence:"))
										]

									+ SVerticalBox::Slot()
										.AutoHeight()
										[
											SNew(SFrameRatePicker)
												.Value(this, &SImageSequenceFrameRateDialog::GetFrameRate)
												.OnValueChanged(this, &SImageSequenceFrameRateDialog::SetFrameRate)
										]
								]
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Bottom)
						.Padding(8)
						[
							SNew(SUniformGridPanel)
								.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
								.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
								.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))

							+ SUniformGridPanel::Slot(0, 0)
								[
									SNew(SButton)
										.HAlign(HAlign_Center)
										.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
										.OnClicked_Lambda([this]() -> FReply
										{
											CloseDialog(true);
											return FReply::Handled();
										})
										.Text(LOCTEXT("OkButtonLabel", "OK"))
								]

							+ SUniformGridPanel::Slot(1, 0)
								[
									SNew(SButton)
										.HAlign(HAlign_Center)
										.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
										.OnClicked_Lambda([this]() -> FReply
										{
											CloseDialog(false);
											return FReply::Handled();
										})
										.Text(LOCTEXT("CancelButtonLabel", "Cancel"))
								]
						]
				]
		];
	}

	bool WasConfirmed() const
	{
		return bConfirmed;
	}

	FFrameRate GetFrameRate() const
	{
		return SelectedFrameRate;
	}

private:
	void SetFrameRate(FFrameRate InFrameRate)
	{
		SelectedFrameRate = InFrameRate;
	}

	void CloseDialog(bool bInConfirmed)
	{
		bConfirmed = bInConfirmed;

		if (TSharedPtr<SWindow> PinnedWindow = Window.Pin())
		{
			PinnedWindow->RequestDestroyWindow();
		}
	}

	TWeakPtr<SWindow> Window;
	FFrameRate SelectedFrameRate;
	bool bConfirmed = false;
};

/**
 * Returns true if the file has an extension registered as an image media source.
 * Uses SpawnMediaSourceForString to check whether the extension produces a UImgMediaSource,
 * with results cached per extension to avoid repeated object creation.
 */
bool HasSupportedImageExtension(const FString& InFilePath)
{
	// Game-thread only: TSet is not thread-safe and Slate drop callbacks always
	// run on the game thread.
	static TSet<FString> SupportedExtensions;
	static TSet<FString> UnsupportedExtensions;

	const FString Extension = FPaths::GetExtension(InFilePath).ToLower();

	if (SupportedExtensions.Contains(Extension))
	{
		return true;
	}

	if (UnsupportedExtensions.Contains(Extension))
	{
		return false;
	}

	// Probe by creating a source via the registered delegate and checking its type.
	UMediaSource* TestSource = UMediaSource::SpawnMediaSourceForString(InFilePath, GetTransientPackage());
	const bool bIsImageSource = TestSource != nullptr && TestSource->IsA<UImgMediaSource>();

	if (bIsImageSource)
	{
		SupportedExtensions.Add(Extension);
	}
	else
	{
		UnsupportedExtensions.Add(Extension);
	}

	return bIsImageSource;
}

/**
 * Owns the OnMediaEvent subscription created when an image sequence is dropped.
 *
 * Lifecycle:
 *  - Created via Attach() and held alive solely by the lambda capture on the player.
 *  - 1st MediaOpened: defer to next tick to show the frame-rate dialog,
 *    so we don't pump a nested Slate input loop inside the broadcast.
 *  - Cancel:        seek nothing, schedule removal.
 *  - Same rate:     seek now, schedule removal.
 *  - Rate changed:  re-open the source; the 2nd MediaOpened seeks and schedules removal.
 *
 * The handle removal is always deferred to next tick to avoid mutating the
 * delegate's invocation list while it is being broadcast. bDone is also set
 * synchronously so any in-flight broadcasts after the latch become no-ops.
 */
class FSequenceOpenHandler : public TSharedFromThis<FSequenceOpenHandler>
{
public:
	using FShowDialogFn = TFunction<bool(FFrameRate& /*OutRate*/, const FFrameRate& /*InDefault*/)>;

	static void Attach(UMediaPlayer* InPlayer, UImgMediaSource* InSource, int32 InSeekFrame, FShowDialogFn InShowDialog);

private:
	FSequenceOpenHandler(UMediaPlayer* InPlayer, UImgMediaSource* InSource, int32 InSeekFrame, FShowDialogFn InShowDialog)
		: WeakPlayer(InPlayer)
		, WeakSource(InSource)
		, SeekFrame(InSeekFrame)
		, ShowDialogFn(MoveTemp(InShowDialog))
	{}

	void HandleMediaEvent(EMediaEvent InEvent);
	void ShowDialogAndSeek();
	void SeekToDroppedFrame(UMediaPlayer& InPlayer, double InFramesPerSecond) const;
	void ScheduleHandleRemoval();

	TWeakObjectPtr<UMediaPlayer> WeakPlayer;
	TWeakObjectPtr<UImgMediaSource> WeakSource;
	int32 SeekFrame = 0;
	FShowDialogFn ShowDialogFn;
	bool bDialogShown = false;
	bool bDone = false;
	FDelegateHandle EventHandle;
};

void FSequenceOpenHandler::Attach(UMediaPlayer* InPlayer, UImgMediaSource* InSource, int32 InSeekFrame, FShowDialogFn InShowDialog)
{
	if (!InPlayer)
	{
		return;
	}

	TSharedRef<FSequenceOpenHandler> Handler = MakeShareable(new FSequenceOpenHandler(InPlayer, InSource, InSeekFrame, MoveTemp(InShowDialog)));

	Handler->EventHandle = InPlayer->OnMediaEvent().AddLambda(
		[Handler](EMediaEvent InEvent)
		{
			Handler->HandleMediaEvent(InEvent);
		});
}

void FSequenceOpenHandler::HandleMediaEvent(EMediaEvent InEvent)
{
	if (InEvent != EMediaEvent::MediaOpened || bDone)
	{
		return;
	}

	if (!bDialogShown)
	{
		bDialogShown = true;

		if (GEditor)
		{
			// Defer to next tick: showing a modal dialog from inside the OnMediaEvent
			// broadcast pumps a nested Slate input loop and can re-enter this handler.
			TSharedRef<FSequenceOpenHandler> Self = AsShared();
			GEditor->GetTimerManager()->SetTimerForNextTick([Self]() { Self->ShowDialogAndSeek(); });
			return;
		}
	}

	// Second open after a frame-rate change. Seek to the dropped frame.
	if (UMediaPlayer* Player = WeakPlayer.Get())
	{
		const float PlayerRate = Player->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE);
		SeekToDroppedFrame(*Player, static_cast<double>(PlayerRate));
	}

	ScheduleHandleRemoval();
}

void FSequenceOpenHandler::ShowDialogAndSeek()
{
	UMediaPlayer* Player = WeakPlayer.Get();
	if (!Player)
	{
		return;
	}

	const float PlayerRate = Player->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE);
	const FFrameRate DefaultFrameRate = UE::MediaViewer::Private::FindClosestCommonFrameRate(PlayerRate);

	FFrameRate SelectedFrameRate;

	if (!ShowDialogFn(SelectedFrameRate, DefaultFrameRate))
	{
		// User canceled - leave the viewer showing the first frame.
		ScheduleHandleRemoval();
		return;
	}

	// If the user changed the rate, re-open with the override applied.
	// The second MediaOpened broadcast will route back through HandleMediaEvent.
	if (!FMath::IsNearlyEqual(SelectedFrameRate.AsDecimal(), DefaultFrameRate.AsDecimal(), 0.01))
	{
		if (UImgMediaSource* Source = WeakSource.Get())
		{
			Source->FrameRateOverride = SelectedFrameRate;
			Player->Close();
			Player->OpenSource(Source); //todo: OpenSourceWithOptions
			return;
		}
	}

	// Same rate as detected - seek to the dropped frame.
	SeekToDroppedFrame(*Player, SelectedFrameRate.AsDecimal());
	ScheduleHandleRemoval();
}

void FSequenceOpenHandler::SeekToDroppedFrame(UMediaPlayer& InPlayer, double InFramesPerSecond) const
{
	if (SeekFrame <= 0 || InFramesPerSecond <= 0.0)
	{
		return;
	}

	// Mid-frame seek lands unambiguously inside the target frame's
	// [N/fps, (N+1)/fps) interval, avoiding tick-rounding off-by-ones.
	const double SeekSeconds = (static_cast<double>(SeekFrame) + 0.5) / InFramesPerSecond;
	InPlayer.Seek(FTimespan::FromSeconds(SeekSeconds));
}

void FSequenceOpenHandler::ScheduleHandleRemoval()
{
	bDone = true;
	
	if (!GEditor)
	{
		return;
	}

	TSharedRef<FSequenceOpenHandler> Self = AsShared();
	GEditor->GetTimerManager()->SetTimerForNextTick([Self]()
	{
		if (UMediaPlayer* Player = Self->WeakPlayer.Get())
		{
			Player->OnMediaEvent().Remove(Self->EventHandle);
		}
	});
}

} // anonymous namespace

namespace UE::MediaViewer::Private
{

void SMediaViewerDropTarget::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

void SMediaViewerDropTarget::Construct(const FArguments& InArgs, const TSharedRef<FMediaViewerDelegates>& InDelegates)
{
	Delegates = InDelegates;
	Position = InArgs._Position;
	bComparisonView = InArgs._bComparisonView;
	bForceComparisonView = InArgs._bForceComparisonView;

	FText UpperMessage;

	if (bComparisonView || (Position == EMediaImageViewerPosition::First && !bForceComparisonView))
	{
		UpperMessage = LOCTEXT("ReplaceImage", "Replace Image");
	}
	else
	{
		UpperMessage = LOCTEXT("CompareImage", "Compare Image");
	}

	ChildSlot
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SDropTarget)
			.OnAllowDrop(this, &SMediaViewerDropTarget::OnAllowDrop)
			.OnIsRecognized(this, &SMediaViewerDropTarget::OnIsRecognized)
			.OnDropped(this, &SMediaViewerDropTarget::OnDropped)
		]
		+ SOverlay::Slot()
		[
			InArgs._Content.Widget
		]
		+ SOverlay::Slot()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Bottom)
			.Padding(0.f, 0.f, 0.f, 5.f)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FStyleColors::Foreground.GetSpecifiedColor())
				.ShadowColorAndOpacity(FStyleColors::Panel.GetSpecifiedColor())
				.ShadowOffset(FVector2D(1.f, 1.f))
				.Visibility(this, &SMediaViewerDropTarget::GetDragDescriptionVisibility)
				.Text(UpperMessage)
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Top)
			.Padding(5.f)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FStyleColors::Foreground.GetSpecifiedColor())
				.ShadowColorAndOpacity(FStyleColors::Panel.GetSpecifiedColor())
				.ShadowOffset(FVector2D(1.f, 1.f))
				.Visibility(this, &SMediaViewerDropTarget::GetDragDescriptionVisibility)
				.Text(LOCTEXT("DropTargetMessage", "Drop supported asset or library item here."))
				.AutoWrapText(true)
			]
		]
	];
}

void SMediaViewerDropTarget::OnDragLeave(const FDragDropEvent& InDragDropEvent)
{
	TSharedPtr<FDragDropOperation> Operation = InDragDropEvent.GetOperation();

	if (Operation.IsValid() && Operation->IsOfType<FDecoratedDragDropOp>())
	{
		TSharedPtr<FDecoratedDragDropOp> DragDropOp = StaticCastSharedPtr<FDecoratedDragDropOp>(Operation);
		DragDropOp->ResetToDefaultToolTip();
	}
}

TArray<FAssetData> SMediaViewerDropTarget::GetAssetsWithImageViewer(TConstArrayView<FAssetData> InAssets)
{
	IMediaViewerModule& MediaViewerModule = IMediaViewerModule::Get();

	TArray<FAssetData> ValidAssets;

	for (const FAssetData& AssetData : InAssets)
	{
		if (MediaViewerModule.HasFactoryFor(AssetData))
		{
			ValidAssets.Add(AssetData);
		}
	}

	return ValidAssets;
}

EVisibility SMediaViewerDropTarget::GetDragDescriptionVisibility() const
{
	return FSlateApplication::Get().IsDragDropping()
		? EVisibility::HitTestInvisible
		: EVisibility::Collapsed;
}

FReply SMediaViewerDropTarget::OnDropped(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	TSharedPtr<FDragDropOperation> DragDropOp = InDragDropEvent.GetOperation();

	if (!DragDropOp.IsValid())
	{
		return FReply::Handled();
	}

	if (TSharedPtr<FMediaViewerLibraryItemDragDropOperation> MediaViewerOp = InDragDropEvent.GetOperationAs<FMediaViewerLibraryItemDragDropOperation>())
	{
		HandleDroppedMediaViewerOp(*MediaViewerOp);
		return FReply::Handled();
	}

	if (TSharedPtr<FExternalDragOperation> FileOp = InDragDropEvent.GetOperationAs<FExternalDragOperation>())
	{
		HandleDroppedFileOp(*FileOp);
		return FReply::Handled();
	}

	TArray<FAssetData> DroppedAssets = AssetUtil::ExtractAssetDataFromDrag(DragDropOp);

	if (!DroppedAssets.IsEmpty())
	{
		HandleDroppedAssets(DroppedAssets);
	}

	return FReply::Handled();
}

void SMediaViewerDropTarget::HandleDroppedMediaViewerOp(const FMediaViewerLibraryItemDragDropOperation& InMediaViewerOp)
{
	TSharedRef<IMediaViewerLibrary> Library = Delegates->GetLibrary.Execute();

	if (TSharedPtr<FMediaViewerLibraryItem> LibraryItem = Library->GetItem(InMediaViewerOp.GetGroupItem().ItemId))
	{
		if (TSharedPtr<FMediaImageViewer> ImageViewer = LibraryItem->CreateImageViewer())
		{
			// Swap images and then replace first image.
			if (bForceComparisonView && Position == EMediaImageViewerPosition::First)
			{
				Delegates->SwapAB.Execute();
				Delegates->SetABView.Execute();
				Delegates->SetABOrientation.Execute(Orient_Horizontal);
			}

			Delegates->SetImageViewer.Execute(Position, ImageViewer.ToSharedRef());
		}
	}
}

void SMediaViewerDropTarget::HandleDroppedAssets(const TArrayView<FAssetData> InDroppedAssets)
{
	IMediaViewerModule& MediaViewerModule = IMediaViewerModule::Get();

	const TArray<FAssetData> ValidAssets = GetAssetsWithImageViewer(InDroppedAssets);
	TArray<TSharedRef<FMediaImageViewer>> ImageViewers;

	for (const FAssetData& AssetData : ValidAssets)
	{
		if (TSharedPtr<FMediaViewerLibraryItem> LibraryItem = MediaViewerModule.CreateLibraryItem(AssetData))
		{
			if (TSharedPtr<FMediaImageViewer> ImageViewer = LibraryItem->CreateImageViewer())
			{
				ImageViewers.Add(ImageViewer.ToSharedRef());

				if (ImageViewers.Num() == static_cast<int32>(EMediaImageViewerPosition::COUNT))
				{
					break;
				}
			}
		}
	}
	
	if (ImageViewers.Num() > 1)
	{
		for (int32 Index = 0; Index < static_cast<int32>(EMediaImageViewerPosition::COUNT); ++Index)
		{
			Delegates->SetImageViewer.Execute(static_cast<EMediaImageViewerPosition>(Index), ImageViewers[Index]);
		}
	}
	else if (ImageViewers.Num() == 1)
	{
		// Swap images and then replace first image.
		if (bForceComparisonView && Position == EMediaImageViewerPosition::First)
		{
			Delegates->SwapAB.Execute();
			Delegates->SetABView.Execute();
			Delegates->SetABOrientation.Execute(Orient_Horizontal);
		}

		Delegates->SetImageViewer.Execute(Position, ImageViewers[0]);
	}
}

void SMediaViewerDropTarget::HandleDroppedFileOp(const FExternalDragOperation& InFileDragDropOp)
{
	if (CreateAndSetImageSequence(InFileDragDropOp))
	{
		return;
	}

	UMediaPlayer* TestMediaPlayer = NewObject<UMediaPlayer>(GetTransientPackage());
	TArray<UFileMediaSource*, TInlineAllocator<2>> Sources;

	for (const FString& FileName : InFileDragDropOp.GetFiles())
	{
		if (!TestMediaPlayer->OpenFile(FileName))
		{
			continue;
		}

		UFileMediaSource* FileMediaSource = NewObject<UFileMediaSource>(GetTransientPackage());
		FileMediaSource->SetFilePath(FileName);

		Sources.Add(FileMediaSource);

		if (Sources.Num() == 2)
		{
			break;
		}
	}

	TestMediaPlayer->Close();

	if (Sources.Num() > 1)
	{
		for (int32 Index = 0; Index < Sources.Num(); ++Index)
		{
			Delegates->SetImageViewer.Execute(
				static_cast<EMediaImageViewerPosition>(Index),
				MakeShared<FMediaSourceImageViewer>(Sources[Index], FText::FromString(Sources[Index]->GetFilePath()))
			);
		}
	}
	else if (Sources.Num() == 1)
	{
		// Swap images and then replace first image.
		if (bForceComparisonView && Position == EMediaImageViewerPosition::First)
		{
			Delegates->SwapAB.Execute();
			Delegates->SetABView.Execute();
			Delegates->SetABOrientation.Execute(Orient_Horizontal);
		}

		Delegates->SetImageViewer.Execute(
			Position,
			MakeShared<FMediaSourceImageViewer>(Sources[0], FText::FromString(Sources[0]->GetFilePath()))
		);
	}
}

bool SMediaViewerDropTarget::CreateAndSetImageSequence(const FExternalDragOperation& InFileDragDropOp)
{
	FString ImageSequencePath;
	const TArray<FString>& Files = InFileDragDropOp.GetFiles();

	if (Files.IsEmpty())
	{
		return false;
	}

	bool bSingleFrameDrop = false;

	// Drag and drop a directory - open as image sequence
	if (Files.Num() == 1 && FPaths::DirectoryExists(Files[0]))
	{
		ImageSequencePath = Files[0];
	}
	// Drag and drop a single image file (standalone or part of a sequence)
	else if (Files.Num() == 1 && HasSupportedImageExtension(Files[0]))
	{
		ImageSequencePath = FPaths::GetPath(Files[0]);
		bSingleFrameDrop = true;
	}
	// Drag and drop more than 2 files (2 would be a comparison view)
	else if (Files.Num() > 2)
	{
		ImageSequencePath = FPaths::GetPath(Files[0]);

		// Sanity check
		if (!FPaths::DirectoryExists(ImageSequencePath))
		{
			return false;
		}

		if (!HasSupportedImageExtension(Files[0]))
		{
			return false;
		}

		for (int32 Index = 1; Index < Files.Num(); ++Index)
		{
			// Check if it's in the same folder.
			if (!FPaths::GetPath(Files[Index]).Equals(ImageSequencePath))
			{
				return false;
			}

			if (!HasSupportedImageExtension(Files[Index]))
			{
				return false;
			}
		}
	}
	else
	{
		return false;
	}

	if (bSingleFrameDrop)
	{
		int32 DroppedFrameNumber = 0;
		int32 FirstFrameNumber = 0;
		const bool bIsSequenceFrame = DetectImageSequence(Files[0], DroppedFrameNumber, FirstFrameNumber);

		// Standalone image (not part of a sequence): load directly as a transient texture.
		// FImgMediaPlayer cannot open a single specific file in a populated folder, so going
		// through it here would scan unrelated siblings (UE-373622).
		if (!bIsSequenceFrame)
		{
			UTexture2D* Texture = FImageUtils::ImportFileAsTexture2D(Files[0]);

			if (!Texture)
			{
				return false;
			}

			// Show the source filename (with extension) in the viewer. Passing an explicit
			// display name avoids renaming the UObject — UObject names cannot contain dots,
			// so the extension would otherwise be lost.
			const FText DisplayName = FText::FromString(FPaths::GetCleanFilename(Files[0]));
			TSharedRef<FTexture2DImageViewer> ImageViewer = MakeShared<FTexture2DImageViewer>(Texture, DisplayName);

			// Swap images and then replace first image.
			if (bForceComparisonView && Position == EMediaImageViewerPosition::First)
			{
				Delegates->SwapAB.Execute();
				Delegates->SetABView.Execute();
				Delegates->SetABOrientation.Execute(Orient_Horizontal);
			}

			Delegates->SetImageViewer.Execute(Position, ImageViewer);

			return true;
		}

		// Sequence frame: open the directory as an ImgMediaSource and seek to the dropped frame.
		const int32 SeekFrame = DroppedFrameNumber - FirstFrameNumber;

		// Use the registered spawn delegate to create the appropriate source type.
		// This verifies the extension is actually handled by ImgMedia (not a video format).
		UMediaSource* MediaSource = UMediaSource::SpawnMediaSourceForString(Files[0], GetTransientPackage());
		UImgMediaSource* ImgSource = Cast<UImgMediaSource>(MediaSource);

		if (!ImgSource)
		{
			return false;
		}

		ImgSource->ClearFlags(RF_Transactional);
		ImgSource->SetFlags(RF_Transient);

		const FText DisplayName = FText::FromString(FPaths::GetCleanFilename(ImageSequencePath));
		TSharedRef<FMediaSourceImageViewer> ImageViewer = MakeShared<FMediaSourceImageViewer>(ImgSource, DisplayName);

		if (UMediaStream* MediaStream = ImageViewer->GetMediaStream())
		{
			if (IMediaStreamPlayer* StreamPlayer = MediaStream->GetPlayer().GetInterface())
			{
				// Disable auto-play so the viewer opens paused.
				FMediaStreamPlayerConfig Config = StreamPlayer->GetPlayerConfig();
				Config.bPlayOnOpen = false;
				StreamPlayer->SetPlayerConfig(Config);

				if (UMediaPlayer* Player = StreamPlayer->GetPlayer())
				{
					FSequenceOpenHandler::Attach(Player, ImgSource, SeekFrame,
						[](FFrameRate& OutRate, const FFrameRate& InDefault)
						{
							return ShowFrameRatePickerDialog(OutRate, InDefault);
						});
				}
			}
		}

		// Swap images and then replace first image.
		if (bForceComparisonView && Position == EMediaImageViewerPosition::First)
		{
			Delegates->SwapAB.Execute();
			Delegates->SetABView.Execute();
			Delegates->SetABOrientation.Execute(Orient_Horizontal);
		}

		Delegates->SetImageViewer.Execute(Position, ImageViewer);

		return true;
	}

	// Directory or multi-file drop: existing save-to-asset workflow.
	const EAppReturnType::Type Return = FMessageDialog::Open(
		EAppMsgType::OkCancel,
		LOCTEXT("ImportAsImageSequence", "Import as Image Media Source?")
	);

	if (Return != EAppReturnType::Ok)
	{
		return false;
	}

	FSaveAssetDialogConfig SaveAssetDialogConfig;
	SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("CreateAssetDialogTitle", "Save As Image Media Source");
	SaveAssetDialogConfig.DefaultPath = "/Game";
	SaveAssetDialogConfig.DefaultAssetName = "ImageMediaSource";
	SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Disallow;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	const FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

	if (SaveObjectPath.IsEmpty())
	{
		return false;
	}

	const FString PackagePath = FPaths::GetBaseFilename(*SaveObjectPath, false);

	UPackage* Package = CreatePackage(*PackagePath);

	if (!Package)
	{
		return false;
	}

	const FString AssetName = FPaths::GetBaseFilename(*SaveObjectPath, true);

	UImgMediaSource* ImageSequence = NewObject<UImgMediaSource>(Package, *AssetName, RF_Public | RF_Standalone | RF_Transactional);

	if (!ImageSequence)
	{
		return false;
	}

	ImageSequence->SetSequencePath(ImageSequencePath);

	FAssetRegistryModule::AssetCreated(ImageSequence);

	IMediaViewerModule& MediaViewerModule = IMediaViewerModule::Get();
	TSharedPtr<FMediaViewerLibraryItem> LibraryItem = MediaViewerModule.CreateLibraryItem(TNotNull<UObject*>(ImageSequence));

	if (!LibraryItem.IsValid())
	{
		return false;
	}

	TSharedPtr<FMediaImageViewer> ImageViewer = LibraryItem->CreateImageViewer();

	if (!ImageViewer.IsValid())
	{
		return false;
	}

	Package->MarkPackageDirty();

	// Swap images and then replace first image.
	if (bForceComparisonView && Position == EMediaImageViewerPosition::First)
	{
		Delegates->SwapAB.Execute();
		Delegates->SetABView.Execute();
		Delegates->SetABOrientation.Execute(Orient_Horizontal);
	}

	Delegates->SetImageViewer.Execute(Position, ImageViewer.ToSharedRef());

	return true;
}

bool DetectImageSequence(const FString& InFilePath, int32& OutDroppedFrameNumber, int32& OutFirstFrameNumber)
{
	if (!HasSupportedImageExtension(InFilePath))
	{
		return false;
	}

	const FString Extension = FPaths::GetExtension(InFilePath).ToLower();
	const FString DroppedFilename = FPaths::GetCleanFilename(InFilePath);
	const FString DirectoryPath = FPaths::GetPath(InFilePath);

	int32 DroppedFrameNumber = 0;
	FString DroppedStem;
	if (!GetFrameNumberAndStem(DroppedFilename, Extension, DroppedFrameNumber, DroppedStem))
	{
		return false;
	}

	TArray<FString> FoundFiles;
	IFileManager::Get().FindFiles(FoundFiles, *DirectoryPath, *Extension);

	int32 MatchCount = 0;
	int32 FirstFrameNumber = DroppedFrameNumber;

	for (const FString& File : FoundFiles)
	{
		// Strict: every same-extension file must have a trailing frame number and share the
		// dropped file's stem. The image media player would otherwise pull these stray files
		// into the sequence and produce broken playback (UE-373622). Bail out and let the
		// caller load the dropped file as a single image instead.
		int32 SiblingFrameNumber = 0;
		FString SiblingStem;

		if (!GetFrameNumberAndStem(File, Extension, SiblingFrameNumber, SiblingStem))
		{
			return false;
		}

		if (!SiblingStem.Equals(DroppedStem, ESearchCase::IgnoreCase))
		{
			return false;
		}

		++MatchCount;
		FirstFrameNumber = FMath::Min(FirstFrameNumber, SiblingFrameNumber);
	}

	// The dropped file plus at least one sibling.
	if (MatchCount < 2)
	{
		return false;
	}

	OutDroppedFrameNumber = DroppedFrameNumber;
	OutFirstFrameNumber = FirstFrameNumber;
	return true;
}

bool SMediaViewerDropTarget::ShowFrameRatePickerDialog(FFrameRate& OutFrameRate, const FFrameRate& InDefaultFrameRate)
{
	if (!GEditor)
	{
		return false;
	}

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("ImageSequenceFrameRateTitle", "Image Sequence Frame Rate"))
		.ClientSize(FVector2D(350, 130))
		.SupportsMinimize(false)
		.SupportsMaximize(false);

	TSharedRef<SImageSequenceFrameRateDialog> Dialog =
		SNew(SImageSequenceFrameRateDialog, Window, InDefaultFrameRate);

	Window->SetContent(Dialog);

	GEditor->EditorAddModalWindow(Window);

	if (Dialog->WasConfirmed())
	{
		OutFrameRate = Dialog->GetFrameRate();
		return true;
	}

	return false;
}

bool SMediaViewerDropTarget::OnAllowDrop(TSharedPtr<FDragDropOperation> InDragDropOperation) const
{
	if (!InDragDropOperation.IsValid())
	{
		return false;
	}

	if (InDragDropOperation->IsOfType<FMediaViewerLibraryItemDragDropOperation>())
	{
		TSharedPtr<FMediaViewerLibraryItemDragDropOperation> LibraryItemDragDropOp = StaticCastSharedPtr<FMediaViewerLibraryItemDragDropOperation>(InDragDropOperation);

		if (!Delegates->GetLibrary.Execute()->GetItem(LibraryItemDragDropOp->GetGroupItem().ItemId).IsValid())
		{
			if (IsHovered())
			{
				LibraryItemDragDropOp->SetToolTip(LOCTEXT("InvalidItem", "Invalid Library Item"), FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")));
			}

			return false;
		}

		return true;
	}

	if (InDragDropOperation->IsOfType<FExternalDragOperation>())
	{
		TSharedPtr<FExternalDragOperation> ExternalDragDropOp = StaticCastSharedPtr<FExternalDragOperation>(InDragDropOperation);
		return ExternalDragDropOp->HasFiles();
	}

	const TArray<FAssetData> DroppedAssets = AssetUtil::ExtractAssetDataFromDrag(InDragDropOperation);
	const TArray<FAssetData> ValidAssets = GetAssetsWithImageViewer(DroppedAssets);

	if (ValidAssets.IsEmpty())
	{
		if (InDragDropOperation->IsOfType<FDecoratedDragDropOp>())
		{
			TSharedPtr<FDecoratedDragDropOp> DecoratedDragDropOp = StaticCastSharedPtr<FDecoratedDragDropOp>(InDragDropOperation);
			DecoratedDragDropOp->SetToolTip(LOCTEXT("NotSupported", "Not Supported"), FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")));
		}

		return false;
	}

	return true;
}

bool SMediaViewerDropTarget::OnIsRecognized(TSharedPtr<FDragDropOperation> InDragDropOperation) const
{
	if (!InDragDropOperation.IsValid())
	{
		return false;
	}

	if (InDragDropOperation->IsOfType<FMediaViewerLibraryItemDragDropOperation>())
	{
		return true;
	}

	if (InDragDropOperation->IsOfType<FExternalDragOperation>())
	{
		return true;
	}

	const TArray<FAssetData> DroppedAssets = AssetUtil::ExtractAssetDataFromDrag(InDragDropOperation);

	return !DroppedAssets.IsEmpty();
}

}

#undef LOCTEXT_NAMESPACE
