// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Misc/FrameRate.h"
#include "SMediaViewer.h"

namespace UE::MediaViewer
{
	struct FMediaViewerDelegates;
}

namespace UE::MediaViewer::Private
{

class FMediaViewerLibraryItemDragDropOperation;

class SMediaViewerDropTarget : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SMediaViewerDropTarget, SCompoundWidget)

	SLATE_BEGIN_ARGS(SMediaViewerDropTarget)
		: _Position(EMediaImageViewerPosition::First)
		, _bComparisonView(false)
		, _bForceComparisonView(false)
		{}
		SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_ARGUMENT(EMediaImageViewerPosition, Position)
		SLATE_ARGUMENT(bool, bComparisonView)
		SLATE_ARGUMENT(bool, bForceComparisonView)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<FMediaViewerDelegates>& InDelegates);

	//~ Begin SWidget
	virtual void OnDragLeave(const FDragDropEvent& InDragDropEvent) override;
	//~ End SWidget

protected:
	/** Filters the given assets and returns the ones with a registered image viewer handler class. */
	static TArray<FAssetData> GetAssetsWithImageViewer(TConstArrayView<FAssetData> InAssets);

	TSharedPtr<FMediaViewerDelegates> Delegates;
	EMediaImageViewerPosition Position;
	bool bComparisonView;
	bool bForceComparisonView;

	EVisibility GetDragDescriptionVisibility() const;

	bool OnAllowDrop(TSharedPtr<FDragDropOperation> InDragDropOperation) const;

	bool OnIsRecognized(TSharedPtr<FDragDropOperation> InDragDropOperation) const;

	FReply OnDropped(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent);

	/**
	 * Takes the dragged library item(s) and displays them in the viewer.
	 * If one is dragged, it will replace the current image viewer.
	 * If 2 or more are dragged, the first 2 will be put in comparison mode, replacing whatever is being shown.
	 */
	void HandleDroppedMediaViewerOp(const FMediaViewerLibraryItemDragDropOperation& InMediaViewerOp);

	/**
	 * Takes the dragged assets and displays them in the viewer, if they have a registered image viewer handler class.
	 * If one is dragged, it will replace the current image viewer.
	 * If 2 or more are dragged, the first 2 will be put in comparison mode, replacing whatever is being shown.
	 */
	void HandleDroppedAssets(const TArrayView<FAssetData> InDroppedAssets);

	/**
	 * Takes the dragged files and attempts to open them.
	 * Single standalone images are loaded as a transient texture; numbered sequence frames and
	 * directories use the image media player; other recognized media types use UMediaPlayer.
	 * If one is dragged, it will replace the current image viewer.
	 * If 2 or more are dragged, the first 2 will be put in comparison mode, replacing whatever is being shown.
	 */
	void HandleDroppedFileOp(const FExternalDragOperation& InFileDragDropOp);

	/**
	 * Looks at the dragged files and determines if it can be turned into an image media source.
	 * If it can, it will ask the user to save the asset and then open it in the media viewer.
	 * Returns true if an asset was created.
	 */
	bool CreateAndSetImageSequence(const FExternalDragOperation& InFileDragDropOp);

	/**
	 * Shows a modal dialog to let the user pick a frame rate for an image sequence.
	 * 
	 * @param OutFrameRate Receives the selected frame rate.
	 * @param InDefaultFrameRate The frame rate to pre-select in the dialog.
	 * 
	 * @return True if the user confirmed.
	 */
	static bool ShowFrameRatePickerDialog(FFrameRate& OutFrameRate, const FFrameRate& InDefaultFrameRate = FFrameRate(24, 1));
};

} // UE::MediaViewer::Private
