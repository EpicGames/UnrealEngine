// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Internal header — used only by SMediaViewerDropTarget.cpp and its unit tests.
// Not for inclusion outside Private/.

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Misc/FrameRate.h"

namespace UE::MediaViewer::Private
{
	/**
	 * Extracts a trailing frame number and the preceding stem from a filename, ignoring the extension.
	 * For example, "render.0042.exr" yields stem "render." and number 42.
	 * Leading zeros are skipped when counting significant digits to avoid spurious overflow rejection
	 * of zero-padded sequence filenames (e.g. "frame_00000042.png").
	 * Out-params are only written on the success (true) path.
	 *
	 * @param InFilename      Filename without path (may include extension).
	 * @param InExtension     Extension string without the leading dot.
	 * @param OutFrameNumber  Receives the parsed frame number on success.
	 * @param OutStem         Receives the prefix before the digit run on success.
	 * @return True if a valid trailing integer was found and fits in int32.
	 */
	bool GetFrameNumberAndStem(const FString& InFilename, const FString& InExtension, int32& OutFrameNumber, FString& OutStem);

	/**
	 * Maps a raw float fps to the closest entry in FCommonFrameRates::GetAll().
	 * Falls back to FFrameRate(24, 1) (the ImgMediaSource default) when the input
	 * is non-positive or no common rate is closer than the initial 24 fps guess.
	 */
	FFrameRate FindClosestCommonFrameRate(float InRate);

	/**
	 * Returns true if the file is part of a numbered image sequence (strict: every sibling
	 * with the same extension must share the stem and have a trailing frame number, and at
	 * least one matching sibling must exist). Out-params are only set on success.
	 *
	 * @param InFilePath The full path to the file.
	 * @param OutDroppedFrameNumber If a sequence is detected, the frame number of the dropped file.
	 * @param OutFirstFrameNumber If a sequence is detected, the smallest frame number among matching siblings.
	 * @return True if the file is part of a sequence (the dropped file plus at least one sibling).
	 */
	bool DetectImageSequence(const FString& InFilePath, int32& OutDroppedFrameNumber, int32& OutFirstFrameNumber);
} // namespace UE::MediaViewer::Private
