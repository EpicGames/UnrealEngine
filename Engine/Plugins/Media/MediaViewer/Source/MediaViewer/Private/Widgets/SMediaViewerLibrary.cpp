// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMediaViewerLibrary.h"

#include "DetailLayoutBuilder.h"
#include "IMediaViewerModule.h"
#include "Library/MediaViewerLibrary.h"
#include "Library/MediaViewerLibraryDynamicGroup.h"
#include "Library/MediaViewerLibraryGroup.h"
#include "Library/MediaViewerLibraryItem.h"
#include "Misc/Guid.h"
#include "Widgets/SMediaViewerLibraryGroup.h"
#include "Widgets/SMediaViewerLibraryItem.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MediaViewerLibrary"

namespace UE::MediaViewer::Private
{

SMediaViewerLibrary::SMediaViewerLibrary()
{
}

SMediaViewerLibrary::~SMediaViewerLibrary()
{
	if (Library.IsValid())
	{
		Library->GetOnChanged().RemoveAll(this);
	}
}

void SMediaViewerLibrary::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

void SMediaViewerLibrary::Construct(const FArguments& InArgs, const IMediaViewerLibraryWidget::FArgs& InMediaViewerLibraryArgs,
	const TSharedRef<FMediaViewerDelegates>& InDelegates)
{
	GroupFilter = InMediaViewerLibraryArgs.GroupFilter;
	Delegates = InDelegates;
	Library = IMediaViewerModule::Get().GetLibrary();
	Library->GetOnChanged().AddSP(this, &SMediaViewerLibrary::OnLibraryChanged);

	SetVisibility(EVisibility::Visible);

	UpdateGroups();

	TreeView = SNew(STreeView<IMediaViewerLibrary::FGroupItem>)
		.TreeItemsSource(&Groups)
		.SelectionMode(ESelectionMode::Single)
		.OnGenerateRow(this, &SMediaViewerLibrary::OnGenerateItemRow)
		.OnGetChildren(this, &SMediaViewerLibrary::OnGetChildren)
		.OnExpansionChanged(this, &SMediaViewerLibrary::OnGroupExpanded);
		
	ChildSlot
	[
		TreeView.ToSharedRef()
	];
}

TSharedRef<SWidget> SMediaViewerLibrary::ToWidget()
{
	return SharedThis(this);
}

TSharedRef<IMediaViewerLibrary> SMediaViewerLibrary::GetLibrary() const
{
	return Library.ToSharedRef();
}

FReply SMediaViewerLibrary::OnMouseButtonDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	// Make sure sidebar blocks widget below it.
	return FReply::Handled();
}

void SMediaViewerLibrary::OnLibraryChanged(TSharedRef<IMediaViewerLibrary> InLibrary, IMediaViewerLibrary::EChangeType InChangeType)
{
	switch (InChangeType)
	{
		case IMediaViewerLibrary::EChangeType::GroupAdded:
		case IMediaViewerLibrary::EChangeType::GroupRemoved:
			UpdateGroups();
			break;

		default:
			// Nothing
			break;
	}

	TreeView->RequestListRefresh();
}

void SMediaViewerLibrary::UpdateGroups()
{
	Groups.Empty(Library->GetGroups().Num());

	TSharedRef<const IMediaViewerLibrary> ConstLibrary = Library.ToSharedRef();

	for (const TSharedRef<const FMediaViewerLibraryGroup>& Group : Library->GetGroups())
	{
		if (!GroupFilter.IsBound() || GroupFilter.Execute(ConstLibrary, Group))
		{
			Groups.Add({Group->GetId(), FGuid()});
		}
	}
}

TSharedRef<class ITableRow> SMediaViewerLibrary::OnGenerateItemRow(IMediaViewerLibrary::FGroupItem InEntry,
	const TSharedRef<STableViewBase>& InOwningTable)
{
	if (InEntry.ItemId.IsValid())
	{
		return SNew(SMediaViewerLibraryItem, InOwningTable, Library.ToSharedRef(), InEntry, Delegates.ToSharedRef());
	}

	return SNew(SMediaViewerLibraryGroup, InOwningTable, Library.ToSharedRef(), InEntry.GroupId);
}

void SMediaViewerLibrary::OnGetChildren(IMediaViewerLibrary::FGroupItem InParent, TArray<IMediaViewerLibrary::FGroupItem>& OutChildren)
{
	if (InParent.ItemId.IsValid())
	{
		return;
	}

	TSharedPtr<FMediaViewerLibraryGroup> Group = Library->GetGroup(InParent.GroupId);

	if (!Group.IsValid())
	{
		return;
	}

	if (Group->IsDynamic())
	{
		StaticCastSharedPtr<FMediaViewerLibraryDynamicGroup>(Group)->UpdateItems();
	}

	OutChildren.Reserve(OutChildren.Num() + Group->GetItems().Num());

	for (const FGuid& ItemId : Group->GetItems())
	{
		if (Library->GetItem(ItemId).IsValid())
		{
			OutChildren.Add({ InParent.GroupId, ItemId });
		}
	}
}

void SMediaViewerLibrary::OnGroupExpanded(IMediaViewerLibrary::FGroupItem InGroup, bool bExpanded)
{
	if (InGroup.ItemId.IsValid())
	{
		return;
	}

	TSharedPtr<FMediaViewerLibraryGroup> Group = Library->GetGroup(InGroup.GroupId);

	if (Group.IsValid() && Group->IsDynamic())
	{
		StaticCastSharedPtr<FMediaViewerLibraryDynamicGroup>(Group)->UpdateItems();
	}
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
