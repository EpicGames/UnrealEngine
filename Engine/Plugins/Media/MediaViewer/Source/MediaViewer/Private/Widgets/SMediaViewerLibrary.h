// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/IMediaViewerLibraryWidget.h"
#include "Widgets/SCompoundWidget.h"

#include "Library/IMediaViewerLibrary.h"
#include "Widgets/MediaViewerLibraryGroupItem.h"
#include "Widgets/Views/STreeView.h"

class STableViewBase;
struct FMediaViewerLibraryEntry;

namespace UE::MediaViewer
{
	class IMediaViewerLibrary;
	struct FMediaViewerDelegates;
}

namespace UE::MediaViewer::Private
{

/**
 * Implementation of @see IMediaViewerLibraryWidget.
 */
class SMediaViewerLibrary : public SCompoundWidget, public IMediaViewerLibraryWidget
{
	SLATE_DECLARE_WIDGET(SMediaViewerLibrary, SCompoundWidget)

public:
	SLATE_BEGIN_ARGS(SMediaViewerLibrary)
		{}
	SLATE_END_ARGS()

	SMediaViewerLibrary();

	virtual ~SMediaViewerLibrary() override;

	void Construct(const FArguments& InArgs, const FArgs& InMediaViewerLibraryArgs, const TSharedRef<FMediaViewerDelegates>& InDelegates);

	void OnLibraryChanged(TSharedRef<IMediaViewerLibrary> InLibrary, IMediaViewerLibrary::EChangeType InChangeType);

	//~ Begin IMediaViewerLibraryWidget
	virtual TSharedRef<SWidget> ToWidget() override;
	virtual TSharedRef<IMediaViewerLibrary> GetLibrary() const override;
	//~ End IMediaViewerLibraryWidget

	//~ Begin SWidget
	virtual FReply OnMouseButtonDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	//~ End SWidget

protected:
	TSharedPtr<FMediaViewerDelegates> Delegates;
	TSharedPtr<IMediaViewerLibrary> Library;
	FGroupFilter GroupFilter;
	TArray<IMediaViewerLibrary::FGroupItem> Groups;
	TSharedPtr<STreeView<IMediaViewerLibrary::FGroupItem>> TreeView;

	void UpdateGroups();

	TSharedRef<class ITableRow> OnGenerateItemRow(IMediaViewerLibrary::FGroupItem InEntry, 
		const TSharedRef<STableViewBase>& InOwningTable);

	void OnGetChildren(IMediaViewerLibrary::FGroupItem InParent, TArray<IMediaViewerLibrary::FGroupItem>& OutChildren);

	void OnGroupExpanded(IMediaViewerLibrary::FGroupItem InGroup, bool bExpanded);
};

} // UE::MediaViewer::Private
