// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaViewerLibraryGroup.h"

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "DetailLayoutBuilder.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "IMediaViewerModule.h"
#include "Library/MediaViewerLibrary.h"
#include "Library/MediaViewerLibraryDynamicGroup.h"
#include "Library/MediaViewerLibraryGroup.h"
#include "Library/MediaViewerLibraryItem.h"
#include "MediaViewerStyle.h"
#include "Misc/Optional.h"
#include "Styling/StyleColors.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/MediaViewerLibraryItemDragDropOperation.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SMediaViewerLibraryGroup"

namespace UE::MediaViewer::Private
{

SMediaViewerLibraryGroup::SMediaViewerLibraryGroup()
{
}

void SMediaViewerLibraryGroup::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

void SMediaViewerLibraryGroup::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwningTable,
	const TSharedRef<IMediaViewerLibrary>& InLibrary, const FGuid& InGroupId)
{
	LibraryWeak = InLibrary;
	GroupId = InGroupId;

	TSharedPtr<FMediaViewerLibraryGroup> Group = InLibrary->GetGroup(InGroupId);

	TSharedRef<SHorizontalBox> Inner = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.Padding(5.f, 5.f, 5.f, 5.f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(this, &SMediaViewerLibraryGroup::GetGroupName)
			.ToolTipText(Group->ToolTip)
			.Font(IDetailLayoutBuilder::GetDetailFontBold())
		];

	if (Group->IsDynamic())
	{
		const FButtonStyle& ButtonStyle = FMediaViewerStyle::Get().GetWidgetStyle<FButtonStyle>("LibraryButtons");

		Inner->AddSlot()
			.AutoWidth()
			.Padding(0.f, 4.f, 5.f, 4.f)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ContentPadding(FMargin(1.f, 3.f))
				.OnClicked(this, &SMediaViewerLibraryGroup::OnRefreshButtonClicked)
				.ButtonStyle(&ButtonStyle)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("SourceControl.Actions.Refresh"))
					.ToolTipText(LOCTEXT("RefreshGroupToolTip", "Refresh the members of this dynamic group."))
					.DesiredSizeOverride(FVector2D(12))
				]
			];

		Inner->AddSlot()
			.AutoWidth()
			.Padding(0.f, 4.f, 5.f, 4.f)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Lock"))
				.ToolTipText(LOCTEXT("DynamicGroupToolTip", "This group is generated dynamically. Items cannot be added or removed."))
				.DesiredSizeOverride(FVector2D(12))
			];
	}
	else if (InLibrary->CanRemoveGroup(Group->GetId()))
	{
		const FButtonStyle& ButtonStyle = FMediaViewerStyle::Get().GetWidgetStyle<FButtonStyle>("LibraryButtons");

		Inner->AddSlot()
			.AutoWidth()
			.Padding(0.f, 4.f, 5.f, 4.f)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ContentPadding(FMargin(1.f, 3.f))
				.OnClicked(this, &SMediaViewerLibraryGroup::OnRemoveButtonClicked)
				.ButtonStyle(&ButtonStyle)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
					.DesiredSizeOverride(FVector2D(12))
					.ToolTipText(LOCTEXT("DeleteGroupTooltip", "Remove this group."))
				]
			];
	}
	else if (Group->GetId() == InLibrary->GetHistoryGroupId())
	{
		const FButtonStyle& ButtonStyle = FMediaViewerStyle::Get().GetWidgetStyle<FButtonStyle>("LibraryButtons");

		Inner->AddSlot()
			.AutoWidth()
			.Padding(0.f, 4.f, 5.f, 4.f)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ContentPadding(FMargin(1.f, 3.f))
				.OnClicked(this, &SMediaViewerLibraryGroup::OnRemoveButtonClicked)
				.ButtonStyle(&ButtonStyle)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
					.DesiredSizeOverride(FVector2D(12))
					.ToolTipText(LOCTEXT("EmptyHistoryGroup", "Remove all items from the history."))
				]
			];
	}

	TSharedRef<SWidget> Box = SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SColorBlock)
			.Color(FStyleColors::Panel.GetSpecifiedColor())
		]
		+ SOverlay::Slot()
		[		
			Inner
		];

	FSuperType::Construct(
		FSuperType::FArguments()
		.ShowWires(false)
		.ShowSelection(false)
		.Content()[Box],
		InOwningTable
	);
}

void SMediaViewerLibraryGroup::OnDragEnter(const FGeometry& InMyGeometry, const FDragDropEvent& InEvent)
{
	FSuperType::OnDragEnter(InMyGeometry, InEvent);

	TSharedPtr<IMediaViewerLibrary> Library = LibraryWeak.Pin();

	if (!Library.IsValid())
	{
		return;
	}

	TSharedPtr<FMediaViewerLibrary> LibraryImpl = StaticCastSharedPtr<FMediaViewerLibrary>(Library);

	if (!LibraryImpl->CanDragDropGroup(GroupId))
	{
		return;
	}

	if (TSharedPtr<FMediaViewerLibraryItemDragDropOperation> ItemDragDrop = InEvent.GetOperationAs<FMediaViewerLibraryItemDragDropOperation>())
	{
		if (!CanAcceptLibraryItem(ItemDragDrop->GetGroupItem()))
		{
			return;
		}
	}
	else if (TSharedPtr<FAssetDragDropOp> AssetDragDrop = InEvent.GetOperationAs<FAssetDragDropOp>())
	{
		if (!CanAcceptAssets(AssetDragDrop->GetAssets()))
		{
			return;
		}
	}
	else
	{
		return;
	}

	SetItemDropZone(EItemDropZone::OntoItem);
}

FReply SMediaViewerLibraryGroup::OnDrop(const FGeometry& InMyGeometry, const FDragDropEvent& InEvent)
{
	SetItemDropZone(TOptional<EItemDropZone>());

	TSharedPtr<IMediaViewerLibrary> Library = LibraryWeak.Pin();

	if (!Library.IsValid())
	{
		return FSuperType::OnDrop(InMyGeometry, InEvent);
	}

	TSharedPtr<FMediaViewerLibrary> LibraryImpl = StaticCastSharedPtr<FMediaViewerLibrary>(Library);

	if (!LibraryImpl->CanDragDropGroup(GroupId))
	{
		return FSuperType::OnDrop(InMyGeometry, InEvent);
	}

	if (TSharedPtr<FMediaViewerLibraryItemDragDropOperation> ItemDragDrop = InEvent.GetOperationAs<FMediaViewerLibraryItemDragDropOperation>())
	{
		OnLibraryItemDropped(ItemDragDrop->GetGroupItem());
	}
	else if (TSharedPtr<FAssetDragDropOp> AssetDragDrop = InEvent.GetOperationAs<FAssetDragDropOp>())
	{
		OnAssetsDropped(AssetDragDrop->GetAssets());
	}
	else
	{
		return FSuperType::OnDrop(InMyGeometry, InEvent);
	}

	return FReply::Handled();
}

FText SMediaViewerLibraryGroup::GetGroupName() const
{
	if (TSharedPtr<IMediaViewerLibrary> Library = LibraryWeak.Pin())
	{
		if (TSharedPtr<FMediaViewerLibraryGroup> Group = Library->GetGroup(GroupId))
		{
			return FText::Format(INVTEXT("{0} ({1})"), Group->Name, FText::AsNumber(Group->GetItems().Num()));
		}
	}

	return LOCTEXT("Error", "Error");
}

FReply SMediaViewerLibraryGroup::OnRemoveButtonClicked()
{
	TSharedPtr<IMediaViewerLibrary> Library = LibraryWeak.Pin();

	if (!Library.IsValid())
	{
		return FReply::Handled();
	}

	TSharedPtr<FMediaViewerLibraryGroup> Group = Library->GetGroup(GroupId);

	if (!Group.IsValid())
	{
		return FReply::Handled();
	}

	if (Group->GetId() == Library->GetHistoryGroupId())
	{
		Library->EmptyGroup(Group->GetId());
	}
	else if (Library->CanRemoveGroup(Group->GetId()))
	{
		Library->RemoveGroup(Group->GetId());
	}

	return FReply::Handled();
}

FReply SMediaViewerLibraryGroup::OnRefreshButtonClicked()
{
	TSharedPtr<IMediaViewerLibrary> Library = LibraryWeak.Pin();

	if (!Library.IsValid())
	{
		return FReply::Handled();
	}

	TSharedPtr<FMediaViewerLibraryGroup> Group = Library->GetGroup(GroupId);

	if (!Group.IsValid() || !Group->IsDynamic())
	{
		return FReply::Handled();
	}

	StaticCastSharedPtr<FMediaViewerLibraryDynamicGroup>(Group)->UpdateItems();

	return FReply::Handled();
}

bool SMediaViewerLibraryGroup::CanAcceptLibraryItem(const IMediaViewerLibrary::FGroupItem& InDraggedGroupItem) const
{
	TSharedPtr<IMediaViewerLibrary> Library = LibraryWeak.Pin();

	if (!Library.IsValid())
	{
		return false;
	}

	if (GroupId == Library->GetHistoryGroupId())
	{
		return false;
	}

	TSharedPtr<FMediaViewerLibrary> LibraryImpl = StaticCastSharedPtr<FMediaViewerLibrary>(Library);

	if (!LibraryImpl->CanDragDropGroup(GroupId))
	{
		return false;
	}

	if (!LibraryImpl->CanDragDropItem(InDraggedGroupItem))
	{
		return false;
	}

	return true;
}

void SMediaViewerLibraryGroup::OnLibraryItemDropped(const IMediaViewerLibrary::FGroupItem& InDroppedGroupItem) const
{
	if (!CanAcceptLibraryItem(InDroppedGroupItem))
	{
		return;
	}

	// Validated by CanAcceptLibraryItem
	TSharedPtr<IMediaViewerLibrary> Library = LibraryWeak.Pin();

	if (GroupId == InDroppedGroupItem.GroupId)
	{
		Library->MoveItemWithinGroup(InDroppedGroupItem, Library->GetGroup(GroupId)->GetItems().Num());
	}
	else if (InDroppedGroupItem.GroupId != Library->GetHistoryGroupId())
	{
		Library->MoveItemToGroup(InDroppedGroupItem, GroupId);
	}
	else
	{
		if (TSharedPtr<FMediaViewerLibraryItem> Item = Library->GetItem(InDroppedGroupItem.ItemId))
		{
			if (TSharedPtr<FMediaViewerLibraryGroup> CurrentGroup = Library->GetItemGroup(InDroppedGroupItem.ItemId))
			{
				CurrentGroup->RemoveItem(InDroppedGroupItem.ItemId);
			}

			Library->AddItemToGroup(Item.ToSharedRef(), GroupId);
		}
	}
}

bool SMediaViewerLibraryGroup::CanAcceptAssets(TConstArrayView<FAssetData> InAssetData) const
{
	TSharedPtr<IMediaViewerLibrary> Library = LibraryWeak.Pin();

	if (!Library.IsValid())
	{
		return false;
	}

	IMediaViewerModule& Module = IMediaViewerModule::Get();

	for (const FAssetData& AssetData : InAssetData)
	{
		TSharedPtr<FMediaViewerLibraryItem> NewItem = Module.CreateLibraryItem(AssetData);

		if (!NewItem.IsValid())
		{
			continue;
		}

		if (Library->FindItemByValue(NewItem->GetItemType(), NewItem->GetStringValue()))
		{
			continue;
		}

		return true;
	}

	return false;
}

void SMediaViewerLibraryGroup::OnAssetsDropped(TConstArrayView<FAssetData> InAssetData)
{
	// Already verified at this point
	TSharedPtr<IMediaViewerLibrary> Library = LibraryWeak.Pin();
	TSharedPtr<FMediaViewerLibraryGroup> Group = Library->GetGroup(GroupId);
	
	IMediaViewerModule& Module = IMediaViewerModule::Get();
	
	for (int32 Index = InAssetData.Num() - 1; Index >= 0; --Index)
	{
		if (TSharedPtr<FMediaViewerLibraryItem> NewItem = Module.CreateLibraryItem(InAssetData[Index]))
		{
			if (!Library->FindItemByValue(NewItem->GetItemType(), NewItem->GetStringValue()).IsValid())
			{
				Library->AddItemToGroup(NewItem.ToSharedRef(), Group->GetId());
			}
		}
	}
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
