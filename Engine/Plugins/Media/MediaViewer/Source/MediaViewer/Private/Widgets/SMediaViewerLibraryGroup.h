// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/STableRow.h"

#include "Library/IMediaViewerLibrary.h"
#include "Misc/Guid.h"
#include "Widgets/IMediaViewerLibraryWidget.h"
#include "Widgets/MediaViewerLibraryGroupItem.h"
#include "Widgets/Views/SListView.h"

struct FMediaViewerLibraryEntry;
struct FMediaViewerLibraryGroup;

namespace UE::MediaViewer
{
	enum class EMediaImageViewerPosition : uint8;
}

namespace UE::MediaViewer::Private
{

class SMediaViewerLibraryGroup : public STableRow<IMediaViewerLibrary::FGroupItem>
{
	using FSuperType = STableRow<IMediaViewerLibrary::FGroupItem>;

	SLATE_DECLARE_WIDGET(SMediaViewerLibraryGroup, SCompoundWidget)

	SLATE_BEGIN_ARGS(SMediaViewerLibraryGroup)
		{}
	SLATE_END_ARGS()

public:
	SMediaViewerLibraryGroup();

	virtual ~SMediaViewerLibraryGroup() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwningTable, 
		const TSharedRef<IMediaViewerLibrary>& InLibrary, const FGuid& InGroupId);

	//~ Begin STableRow
	virtual void OnDragEnter(const FGeometry& InMyGeometry, const FDragDropEvent& InEvent) override;
	virtual FReply OnDrop(const FGeometry& InMyGeometry, const FDragDropEvent& InEvent) override;
	//~ End STableRow

protected:
	TWeakPtr<IMediaViewerLibrary> LibraryWeak;
	FGuid GroupId;

	FText GetGroupName() const;

	FReply OnRemoveButtonClicked();

	FReply OnRefreshButtonClicked();

	bool CanAcceptLibraryItem(const IMediaViewerLibrary::FGroupItem& InDraggedGroupItem) const;
	void OnLibraryItemDropped(const IMediaViewerLibrary::FGroupItem& InDroppedGroupItem) const;

	bool CanAcceptAssets(TConstArrayView<FAssetData> InAssetData) const;
	void OnAssetsDropped(TConstArrayView<FAssetData> InAssetData);
};

} // UE::MediaViewer::Private
