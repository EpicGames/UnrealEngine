// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaViewerLibraryItem.h"

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "DetailLayoutBuilder.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "ImageViewer/MediaImageViewer.h"
#include "IMediaViewerModule.h"
#include "Library/MediaViewerLibrary.h"
#include "Library/MediaViewerLibraryGroup.h"
#include "Library/MediaViewerLibraryItem.h"
#include "MediaViewer.h"
#include "MediaViewerDelegates.h"
#include "MediaViewerStyle.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/MediaViewerLibraryItemAssetDragDropOperation.h"
#include "Widgets/MediaViewerLibraryItemDragDropOperation.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SMediaViewerLibraryItem"

namespace UE::MediaViewer::Private
{

SMediaViewerLibraryItem::SMediaViewerLibraryItem()
{
}

void SMediaViewerLibraryItem::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

void SMediaViewerLibraryItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwningTable, 
	const TSharedRef<IMediaViewerLibrary>& InLibrary, const IMediaViewerLibrary::FGroupItem& InGroupItem,
	const TSharedRef<FMediaViewerDelegates>& InDelegates)
{
	LibraryWeak = InLibrary;
	GroupItem = InGroupItem;
	Delegates = InDelegates;

	// Validated by the Library widget
	Item = InLibrary->GetItem(InGroupItem.ItemId);

	TSharedPtr<FMediaViewerLibraryGroup> Group = InLibrary->GetGroup(GroupItem.GroupId);

	SetCursor(EMouseCursor::Hand);

	ThumbnailBrush = Item->CreateThumbnail();

	TSharedRef<SHorizontalBox> Box = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.f, 5.f, 5.f, 5.f)
		.VAlign(VAlign_Center)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.f)
			[
				SNew(SImage)
				.Image(this, &SMediaViewerLibraryItem::GetThumbnail)
				.DesiredSizeOverride(FVector2D(36.f))
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SColorBlock)
				.Size(FVector2D(40.f, 2.f))
				.Color(Item->GetItemTypeColor().GetSpecifiedColor())
				.CornerRadius(FVector4(0.f, 0.f, 2.f, 2.f))
			]
		] 
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.Padding(3.f, 5.f, 0.f, 5.f)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SVerticalBox)
			.Clipping(EWidgetClipping::ClipToBounds)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(0.f, 0.f, 0.f, 5.f)
			[
				SNew(STextBlock)
				.Text(Item->Name)
				.ToolTipText(Item->ToolTip)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(Item->GetItemTypeDisplayName())
			]
		];

	const FButtonStyle& ButtonStyle = FMediaViewerStyle::Get().GetWidgetStyle<FButtonStyle>("LibraryButtonStyle");
	const FCheckBoxStyle& CheckBoxStyle = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("DetailsView.SectionButton");

	Box->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(3.f, 5.f, 0.f, 5.f)
		[
			SNew(SCheckBox)
			.Style(&CheckBoxStyle)
			.Visibility(this, &SMediaViewerLibraryItem::GetHoveredOrActiveVisibility)
			.Padding(FMargin(9.f, 5.f))
			.IsChecked(this, &SMediaViewerLibraryItem::IsActiveState, EMediaImageViewerPosition::First)
			.OnCheckStateChanged(this, &SMediaViewerLibraryItem::OnUseButtonClicked, EMediaImageViewerPosition::First)
			[
				SNew(STextBlock)
				.Text(INVTEXT("A"))
				.ToolTipText(LOCTEXT("SetA", "Set as A image."))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];

	Box->AddSlot()
		.AutoWidth()
		.Padding(3.f, 5.f, 3.f, 5.f)
		.VAlign(VAlign_Center)
		[
			SNew(SCheckBox)
			.Style(&CheckBoxStyle)
			.Visibility(this, &SMediaViewerLibraryItem::GetHoveredOrActiveVisibility)
			.Padding(FMargin(9.f, 5.f))
			.IsChecked(this, &SMediaViewerLibraryItem::IsActiveState, EMediaImageViewerPosition::Second)
			.OnCheckStateChanged(this, &SMediaViewerLibraryItem::OnUseButtonClicked, EMediaImageViewerPosition::Second)
			[
				SNew(STextBlock)
				.Text(INVTEXT("B"))
				.ToolTipText(LOCTEXT("SetB", "Set as B image."))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];

	if (InLibrary->CanRemoveItemFromGroup(InGroupItem))
	{
		Box->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.f, 5.f, 3.f, 5.f)
			[
				SNew(SButton)
				.Visibility(this, &SMediaViewerLibraryItem::GetHoveredOrActiveVisibility)
				.ContentPadding(FMargin(3.f, 5.f, 3.f, 6.f))
				.OnClicked(this, &SMediaViewerLibraryItem::OnRemoveButtonClicked)
				.ButtonStyle(&ButtonStyle)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
					.ToolTipText(LOCTEXT("RemoveImage", "Remove this from the Library."))
					.DesiredSizeOverride(FVector2D(12))
				]
			];
	}

	TSharedRef<SOverlay> Overlay = SNew(SOverlay)
		+ SOverlay::Slot()
		.Padding(0.f, 0.f, 2.f, 0.f)
		[
			SNew(SColorBlock)
			.Color(this, &SMediaViewerLibraryItem::GetBorderColor)
			.CornerRadius(2.f)
		]
		+ SOverlay::Slot()
		.Padding(2.f, 2.f, 4.f, 2.f)
		[
			SNew(SColorBlock)
			.Color(this, &SMediaViewerLibraryItem::GetFillColor)
		]
		+ SOverlay::Slot()
		.Padding(6.f, 2.f, 4.f, 2.f)
		[
			Box
		];

	FSuperType::Construct(
		FSuperType::FArguments()
			.OnDragDetected(this, &SMediaViewerLibraryItem::OnDragDetected)
			.ShowWires(false)
			.ShowSelection(false)
			.Content()[Overlay],
		InOwningTable
	);
}

FLinearColor SMediaViewerLibraryItem::GetBorderColor() const
{
	for (int32 Index = 0; Index < static_cast<int32>(EMediaImageViewerPosition::COUNT); ++Index)
	{
		if (IsActive(static_cast<EMediaImageViewerPosition>(Index)))
		{
			return FStyleColors::Primary.GetSpecifiedColor();
		}
	}

	return GetFillColor();
}

FLinearColor SMediaViewerLibraryItem::GetFillColor() const
{
	if (IsHovered())
	{
		return FStyleColors::Panel.GetSpecifiedColor();
	}

	return FStyleColors::Background.GetSpecifiedColor();
}

bool SMediaViewerLibraryItem::IsActive(EMediaImageViewerPosition InPosition) const
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(InPosition))
	{
		return ImageViewer->GetInfo().Id == Item->GetId();
	}

	return false;
}

EVisibility SMediaViewerLibraryItem::GetHoveredOrActiveVisibility() const
{
	if (IsHovered())
	{
		return EVisibility::Visible;
	}

	for (int32 Index = 0; Index < static_cast<int32>(EMediaImageViewerPosition::COUNT); ++Index)
	{
		if (IsActive(static_cast<EMediaImageViewerPosition>(Index)))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

void SMediaViewerLibraryItem::SetImageViewer(EMediaImageViewerPosition InPosition)
{
	if (!Item.IsValid())
	{
		return;
	}

	TSharedPtr<FMediaImageViewer> ImageViewer = Item->CreateImageViewer();

	if (!ImageViewer.IsValid())
	{
		return;
	}

	Delegates->SetImageViewer.Execute(InPosition, ImageViewer.ToSharedRef());
}

void SMediaViewerLibraryItem::ClearImageViewer(EMediaImageViewerPosition InPosition)
{
	Delegates->ClearImageViewer.Execute(InPosition);
}

ECheckBoxState SMediaViewerLibraryItem::IsActiveState(EMediaImageViewerPosition InPosition) const
{
	if (IsActive(InPosition))
	{
		return ECheckBoxState::Checked;
	}

	return ECheckBoxState::Unchecked;
}

void SMediaViewerLibraryItem::OnUseButtonClicked(ECheckBoxState InCheckState, EMediaImageViewerPosition InPosition)
{
	switch (InCheckState)
	{
		case ECheckBoxState::Checked:
			SetImageViewer(InPosition);
			break;

		case ECheckBoxState::Unchecked:
			ClearImageViewer(InPosition);
			break;

		default:
			// Do nothing
			break;
	}
}

FReply SMediaViewerLibraryItem::OnRemoveButtonClicked()
{
	TSharedPtr<IMediaViewerLibrary> Library = LibraryWeak.Pin();

	if (!Library.IsValid())
	{
		return FReply::Handled();
	}

	Library->RemoveItemFromGroup(GroupItem);

	return FReply::Handled();
}

FReply SMediaViewerLibraryItem::OnDragDetected(const FGeometry& InMyGeometry, const FPointerEvent& InPointerEvent)
{
	if (!Item.IsValid())
	{
		return FReply::Handled();
	}

	TSharedPtr<IMediaViewerLibrary> Library = LibraryWeak.Pin();

	if (!Library.IsValid())
	{
		return FReply::Handled();
	}

	if (Item->AsAsset())
	{
		return FReply::Handled().BeginDragDrop(MakeShared<FMediaViewerLibraryItemAssetDragDropOperation>(
			Library.ToSharedRef(),
			GroupItem
		));
	}

	return FReply::Handled().BeginDragDrop(MakeShared<FMediaViewerLibraryItemDragDropOperation>(
		Library.ToSharedRef(),
		GroupItem
	));
}

bool SMediaViewerLibraryItem::CanAcceptLibraryItem(const IMediaViewerLibrary::FGroupItem& InDraggedGroupItem) const
{
	if (InDraggedGroupItem.ItemId == GroupItem.ItemId)
	{
		return false;
	}

	TSharedPtr<IMediaViewerLibrary> Library = LibraryWeak.Pin();

	if (!Library.IsValid())
	{
		return false;
	}

	if (GroupItem.GroupId == Library->GetHistoryGroupId())
	{
		return false;
	}

	TSharedPtr<FMediaViewerLibrary> LibraryImpl = StaticCastSharedPtr<FMediaViewerLibrary>(Library);

	if (!LibraryImpl->CanDragDropGroup(GroupItem.GroupId))
	{
		return false;
	}

	if (!LibraryImpl->CanDragDropGroup(InDraggedGroupItem.GroupId))
	{
		return false;
	}

	TSharedPtr<FMediaViewerLibraryItem> DraggedItem = Library->GetItem(InDraggedGroupItem.ItemId);

	if (!DraggedItem.IsValid())
	{
		return false;
	}

	TSharedPtr<FMediaViewerLibraryGroup> MyGroup = Library->GetGroup(GroupItem.GroupId);

	if (!MyGroup.IsValid())
	{
		return false;
	}

	return MyGroup->GetId() != Library->GetHistoryGroupId();
}

void SMediaViewerLibraryItem::OnLibraryItemDropped(const IMediaViewerLibrary::FGroupItem& InDroppedGroupItem) const
{
	if (!CanAcceptLibraryItem(InDroppedGroupItem))
	{
		return;
	}

	// Validated by CanAcceptLibraryItem
	TSharedPtr<IMediaViewerLibrary> Library = LibraryWeak.Pin();
	TSharedPtr<FMediaViewerLibraryGroup> MyGroup = Library->GetGroup(GroupItem.GroupId);
	const int32 MyGroupIndex = MyGroup->FindItemIndex(GroupItem.ItemId);


	if (InDroppedGroupItem.GroupId == GroupItem.GroupId)
	{
		Library->MoveItemWithinGroup(
			InDroppedGroupItem,
			GetItemDropZone().Get(EItemDropZone::OntoItem) == EItemDropZone::AboveItem
				? MyGroupIndex
				: MyGroupIndex + 1
		);
	}
	else if (InDroppedGroupItem.GroupId != Library->GetHistoryGroupId())
	{
		Library->MoveItemToGroup(
			InDroppedGroupItem,
			GroupItem.GroupId,
			GetItemDropZone().Get(EItemDropZone::OntoItem) == EItemDropZone::AboveItem
				? MyGroupIndex
				: MyGroupIndex + 1
		);
	}
	else
	{
		if (TSharedPtr<FMediaViewerLibraryGroup> CurrentGroup = Library->GetItemGroup(InDroppedGroupItem.ItemId))
		{
			CurrentGroup->RemoveItem(InDroppedGroupItem.ItemId);
		}

		Library->AddItemToGroup(
			Library->GetItem(InDroppedGroupItem.ItemId).ToSharedRef(),
			GroupItem.GroupId,
			GetItemDropZone().Get(EItemDropZone::OntoItem) == EItemDropZone::AboveItem
				? MyGroupIndex
				: MyGroupIndex + 1
		);
	}
}

void SMediaViewerLibraryItem::OnDragEnter(const FGeometry& InMyGeometry, const FDragDropEvent& InEvent)
{
	FSuperType::OnDragEnter(InMyGeometry, InEvent);

	if (TSharedPtr<FMediaViewerLibraryItemDragDropOperation> ItemDragDrop = InEvent.GetOperationAs<FMediaViewerLibraryItemDragDropOperation>())
	{
		if (!CanAcceptLibraryItem(ItemDragDrop->GetGroupItem()))
		{
			return;
		}
	}
	else if (TSharedPtr<FAssetDragDropOp> AssetDragDrop = InEvent.GetOperationAs<FAssetDragDropOp>())
	{
		if (!CanAcceptAssets(AssetDragDrop->GetAssets()))
		{
			return;
		}
	}
	else
	{
		return;
	}

	const float Top = GetTickSpaceGeometry().AbsolutePosition.Y;
	const float Bottom = Top + GetTickSpaceGeometry().GetAbsoluteSize().Y;
	const float Mouse = InEvent.GetScreenSpacePosition().Y;

	if (Mouse > Top && Mouse < Bottom)
	{
		if (Mouse < (Top + 5))
		{
			SetItemDropZone(EItemDropZone::AboveItem);
		}
		else if (Mouse > (Bottom - 5))
		{
			SetItemDropZone(EItemDropZone::BelowItem);
		}
		else
		{
			SetItemDropZone(EItemDropZone::OntoItem);
		}
	}
}

FReply SMediaViewerLibraryItem::OnDrop(const FGeometry& InMyGeometry, const FDragDropEvent& InEvent)
{
	TSharedPtr<FAssetDragDropOp> AssetDragDrop = InEvent.GetOperationAs<FAssetDragDropOp>();

	if (TSharedPtr<FMediaViewerLibraryItemDragDropOperation> ItemDragDrop = InEvent.GetOperationAs<FMediaViewerLibraryItemDragDropOperation>())
	{
		OnLibraryItemDropped(ItemDragDrop->GetGroupItem());
	}
	else if (AssetDragDrop.IsValid())
	{
		OnAssetsDropped(AssetDragDrop->GetAssets());
	}

	SetItemDropZone(TOptional<EItemDropZone>());

	return FReply::Handled();
}

bool SMediaViewerLibraryItem::CanAcceptAssets(TConstArrayView<FAssetData> InAssetData) const
{
	TSharedPtr<IMediaViewerLibrary> Library = LibraryWeak.Pin();

	if (!Library.IsValid())
	{
		return false;
	}

	IMediaViewerModule& Module = IMediaViewerModule::Get();

	for (const FAssetData& AssetData : InAssetData)
	{
		TSharedPtr<FMediaViewerLibraryItem> NewItem = Module.CreateLibraryItem(AssetData);

		if (!NewItem.IsValid())
		{
			continue;
		}

		if (Library->FindItemByValue(NewItem->GetItemType(), NewItem->GetStringValue()))
		{
			continue;
		}

		return true;
	}

	return false;
}

void SMediaViewerLibraryItem::OnAssetsDropped(TConstArrayView<FAssetData> InAssetData)
{
	// Already verified at this point
	TSharedPtr<IMediaViewerLibrary> Library = LibraryWeak.Pin();

	TSharedPtr<FMediaViewerLibraryGroup> Group = Library->GetGroup(GroupItem.GroupId);

	if (!Group.IsValid())
	{
		return;
	}

	const int32 ItemIndex = Group->FindItemIndex(GroupItem.ItemId);

	if (ItemIndex == INDEX_NONE)
	{
		return;
	}
	
	IMediaViewerModule& Module = IMediaViewerModule::Get();	
	const EItemDropZone DropZone = GetItemDropZone().Get(EItemDropZone::OntoItem);

	for (int32 Index = InAssetData.Num() - 1; Index >= 0; --Index)
	{
		if (TSharedPtr<FMediaViewerLibraryItem> NewItem = Module.CreateLibraryItem(InAssetData[Index]))
		{
			if (!Library->FindItemByValue(NewItem->GetItemType(), NewItem->GetStringValue()).IsValid())
			{
				Library->AddItemToGroup(
					NewItem.ToSharedRef(),
					GroupItem.GroupId,
					DropZone == EItemDropZone::AboveItem
						? ItemIndex
						: ItemIndex + 1
				);
			}
		}
	}
}

const FSlateBrush* SMediaViewerLibraryItem::GetThumbnail() const
{
	if (ThumbnailBrush.IsValid())
	{
		UObject* ResourceObject = ThumbnailBrush->GetResourceObject();

		if (ResourceObject && ResourceObject->IsValidLowLevel() && IsValid(ResourceObject))
		{
			return ThumbnailBrush.Get();
		}
	}

	return FAppStyle::Get().GetBrush("SourceControl.StatusIcon.Unknown");
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
