// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/STableRow.h"

#include "Library/IMediaViewerLibrary.h"
#include "Misc/Guid.h"
#include "Widgets/IMediaViewerLibraryWidget.h"
#include "Widgets/MediaViewerLibraryGroupItem.h"
#include "Widgets/Views/SListView.h"

enum class EItemDropZone;
struct FMediaViewerLibraryEntry;
struct FMediaViewerLibraryGroup;
struct FMediaViewerLibraryItem;
struct FSlateBrush;

namespace UE::MediaViewer
{
	enum class EMediaImageViewerPosition : uint8;
	struct FMediaViewerDelegates;
}

namespace UE::MediaViewer::Private
{

class SMediaViewerLibraryItem : public STableRow<IMediaViewerLibrary::FGroupItem>
{
	using FSuperType = STableRow<IMediaViewerLibrary::FGroupItem>;

	SLATE_DECLARE_WIDGET(SMediaViewerLibraryItem, SCompoundWidget)

	SLATE_BEGIN_ARGS(SMediaViewerLibraryItem)
		{}
	SLATE_END_ARGS()

public:
	SMediaViewerLibraryItem();

	virtual ~SMediaViewerLibraryItem() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwningTable,
		const TSharedRef<IMediaViewerLibrary>& InLibrary, const IMediaViewerLibrary::FGroupItem& InGroupItem, 
		const TSharedRef<FMediaViewerDelegates>& InDelegates);

	//~ Begin STableRow
	virtual void OnDragEnter(const FGeometry& InMyGeometry, const FDragDropEvent& InEvent) override;
	virtual FReply OnDrop(const FGeometry& InMyGeometry, const FDragDropEvent& InEvent) override;
	//~ End STableRow

protected:
	TWeakPtr<IMediaViewerLibrary> LibraryWeak;
	IMediaViewerLibrary::FGroupItem GroupItem;
	TSharedPtr<FMediaViewerDelegates> Delegates;
	TSharedPtr<FMediaViewerLibraryItem> Item;
	TSharedPtr<FSlateBrush> ThumbnailBrush;

	FLinearColor GetBorderColor() const;

	FLinearColor GetFillColor() const;

	bool IsActive(EMediaImageViewerPosition InPosition) const;

	EVisibility GetHoveredOrActiveVisibility() const;

	void SetImageViewer(EMediaImageViewerPosition InPosition);

	void ClearImageViewer(EMediaImageViewerPosition InPosition);

	ECheckBoxState IsActiveState(EMediaImageViewerPosition InPosition) const;
	void OnUseButtonClicked(ECheckBoxState InCheckState, EMediaImageViewerPosition InPosition);

	FReply OnRemoveButtonClicked();

	FReply OnDragDetected(const FGeometry& InMyGeometry, const FPointerEvent& InPointerEvent);

	bool CanAcceptLibraryItem(const IMediaViewerLibrary::FGroupItem& InDraggedGroupItem) const;
	void OnLibraryItemDropped(const IMediaViewerLibrary::FGroupItem& InDroppedGroupItem) const;

	bool CanAcceptAssets(TConstArrayView<FAssetData> InAssetData) const;
	void OnAssetsDropped(TConstArrayView<FAssetData> InAssetData);

	const FSlateBrush* GetThumbnail() const;
};

} // UE::MediaViewer::Private
