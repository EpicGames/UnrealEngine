// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaViewerLibraryPrivate.h"

#include "ImageViewers/ColorImageViewer.h"
#include "LevelEditor.h"
#include "Library/MediaViewerLibrary.h"
#include "Library/MediaViewerLibraryGroup.h"
#include "Library/MediaViewerLibraryIni.h"
#include "Library/MediaViewerLibraryItem.h"
#include "MediaViewer.h"
#include "MediaViewerDelegates.h"
#include "Misc/Optional.h"
#include "Widgets/SMediaViewer.h"
#include "Widgets/SMediaViewerLibrary.h"

#define LOCTEXT_NAMESPACE "SMediaViewerLibraryPrivate"

namespace UE::MediaViewer::Private
{

SMediaViewerLibraryPrivate::SMediaViewerLibraryPrivate()
{
}

void SMediaViewerLibraryPrivate::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

void SMediaViewerLibraryPrivate::Construct(const FArguments& InArgs, const TSharedRef<FMediaViewerDelegates>& InDelegates)
{
	IMediaViewerLibraryWidget::FArgs Args;
	Args.GroupFilter = InArgs._GroupFilter;

	ChildSlot
	[
		SAssignNew(Library, SMediaViewerLibrary, Args, InDelegates)
	];
}

TSharedRef<IMediaViewerLibrary> SMediaViewerLibraryPrivate::GetLibrary() const
{
	return Library->GetLibrary();
}

void SMediaViewerLibraryPrivate::OnImageViewerOpened(const TSharedRef<FMediaImageViewer>& InImageViewer)
{
	TSharedRef<IMediaViewerLibrary> LibraryImpl = Library->GetLibrary();

	TSharedPtr<FMediaViewerLibraryGroup> HistoryGroup = LibraryImpl->GetGroup(LibraryImpl->GetHistoryGroupId());

	if (!HistoryGroup.IsValid())
	{
		return;
	}

	TSharedPtr<FMediaViewerLibraryItem> Item = LibraryImpl->GetItem(InImageViewer->GetInfo().Id);

	if (!Item.IsValid())
	{
		Item = InImageViewer->CreateLibraryItem();

		if (!Item.IsValid())
		{
			return;
		}

		TSharedPtr<FMediaViewerLibraryItem> ExistingItem = LibraryImpl->FindItemByValue(Item->GetItemType(), Item->GetStringValue());

		if (ExistingItem)
		{
			Item = ExistingItem;
			InImageViewer->UpdateId(ExistingItem->GetId());
		}
		else
		{
			LibraryImpl->AddItem(Item.ToSharedRef());
		}
	}

	LibraryImpl->RemoveItemFromGroup({HistoryGroup->GetId(), Item->GetId()});

	LibraryImpl->AddItemToGroup(Item.ToSharedRef(), HistoryGroup->GetId(), /* Add to the top of the group */ 0);

	while (HistoryGroup->GetItems().Num() > UE::MediaViewer::Private::MaxHistoryEntries)
	{
		LibraryImpl->RemoveItemFromGroup({HistoryGroup->GetId(), HistoryGroup->GetItems().Last()});
	}
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
