// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Widgets/IMediaViewerLibraryWidget.h"

struct FMediaViewerLibraryGroup;

namespace UE::MediaViewer
{
	class FMediaImageViewer;
	class IMediaViewerLibrary;
	enum class EMediaImageViewerPosition : uint8;
	struct FMediaViewerDelegates;
}

namespace UE::MediaViewer::Private
{

class SMediaViewerLibrary;

class SMediaViewerLibraryPrivate : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SMediaViewerLibraryPrivate, SCompoundWidget)

	SLATE_BEGIN_ARGS(SMediaViewerLibraryPrivate)
		{}
		SLATE_EVENT(IMediaViewerLibraryWidget::FGroupFilter, GroupFilter)
	SLATE_END_ARGS()

public:
	SMediaViewerLibraryPrivate();
	virtual ~SMediaViewerLibraryPrivate() = default;

	void Construct(const FArguments& InArgs, const TSharedRef<FMediaViewerDelegates>& InDelegates);

	TSharedRef<IMediaViewerLibrary> GetLibrary() const;

	void OnImageViewerOpened(const TSharedRef<FMediaImageViewer>& InImageViewer);

protected:
	TSharedPtr<SMediaViewerLibrary> Library;
};

} // UE::MediaViewer::Private
