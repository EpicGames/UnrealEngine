// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaViewerTab.h"

#include "EngineAnalytics.h"
#include "ImageViewers/ColorImageViewer.h"
#include "ImageViewers/NullImageViewer.h"
#include "Library/MediaViewerLibrary.h"
#include "Widgets/SMediaViewer.h"
#include "Widgets/SMediaViewerLibrary.h"

#define LOCTEXT_NAMESPACE "MediaViewerTab"

namespace UE::MediaViewer
{

SMediaViewerTab::SMediaViewerTab()
{
}

void SMediaViewerTab::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

SMediaViewerTab::~SMediaViewerTab()
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MediaViewer"), TEXT("Action"), TEXT("ClosedTab"));
	}
}

void SMediaViewerTab::Construct(const FArguments& InArgs, const FMediaViewerArgs& InMediaViewerArgs)
{
	TSharedPtr<FMediaImageViewer> ImageViewerLeft = InArgs._ImageViewerLeft;
	TSharedPtr<FMediaImageViewer> ImageViewerRight = InArgs._ImageViewerRight;

	if (!ImageViewerLeft.IsValid())
	{
		ImageViewerLeft = Private::FNullImageViewer::GetNullImageViewer();
	}

	if (!ImageViewerRight.IsValid())
	{
		ImageViewerRight = Private::FNullImageViewer::GetNullImageViewer();
	}

	ChildSlot
	[
		SAssignNew(Viewer, Private::SMediaViewer, SharedThis(this), InMediaViewerArgs, ImageViewerLeft.ToSharedRef(), ImageViewerRight.ToSharedRef())
	];

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MediaViewer"), TEXT("Action"), TEXT("OpenedTab"));
	}
}

const FMediaViewerArgs& SMediaViewerTab::GetArgs() const
{
	return Viewer->GetArgs();
}

TSharedRef<IMediaViewerLibrary> SMediaViewerTab::GetLibrary() const
{
	return Viewer->GetLibrary();
}

TSharedPtr<FMediaImageViewer> SMediaViewerTab::GetImageViewer(EMediaImageViewerPosition InPosition) const
{
	return Viewer->GetImageViewer(InPosition);
}

void SMediaViewerTab::SetImageViewer(EMediaImageViewerPosition InPosition, TSharedPtr<FMediaImageViewer> InImageViewer)
{
	if (!InImageViewer.IsValid())
	{
		InImageViewer = Private::FNullImageViewer::GetNullImageViewer();
	}

	Viewer->SetImageViewer(InPosition, InImageViewer.ToSharedRef());
}

TSharedRef<Private::SMediaViewer> SMediaViewerTab::GetViewer() const
{
	return Viewer.ToSharedRef();
}

} // UE::MediaViewer

#undef LOCTEXT_NAMESPACE
