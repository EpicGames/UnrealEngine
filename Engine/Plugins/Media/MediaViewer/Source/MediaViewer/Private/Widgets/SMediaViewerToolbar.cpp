// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaViewerToolbar.h"

#include "DetailLayoutBuilder.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/SToolBarButtonBlock.h"
#include "ImageViewer/MediaImageViewer.h"
#include "ImageViewers/NullImageViewer.h"
#include "IMediaViewerModule.h"
#include "Library/MediaViewerLibrary.h"
#include "MediaViewer.h"
#include "MediaViewerCommands.h"
#include "MediaViewerStyle.h"
#include "MediaViewerUtils.h"
#include "SEditorViewportToolBarMenu.h"
#include "Styling/SlateTypes.h"
#include "Styling/ToolBarStyle.h"
#include "UObject/StructOnScope.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/SMediaImageViewerDetails.h"
#include "Widgets/SMediaViewer.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SMediaViewerToolbar"

namespace UE::MediaViewer::Private
{

SMediaViewerToolbar::SMediaViewerToolbar()
{
}

void SMediaViewerToolbar::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

void SMediaViewerToolbar::Construct(const FArguments& InArgs, const TSharedRef<FMediaViewerDelegates>& InDelegates)
{
	using namespace UE::MediaViewer;

	Delegates = InDelegates;

	for (int32 Index = 0; Index < static_cast<int32>(EMediaImageViewerPosition::COUNT); ++Index)
	{
		ImageDetails[Index] = SNullWidget::NullWidget;
	}

	MediaViewerSettingsView = FMediaViewerUtils::CreateStructDetailsView(
		MakeShared<FStructOnScope>(FMediaViewerSettings::StaticStruct(), reinterpret_cast<uint8*>(InDelegates->GetSettings.Execute())),
		LOCTEXT("BackgroundTexture", "Background Texture"),
		this
	);

	MediaViewerSettingsWidget = SNew(SBox)
		.Padding(3.f)
		[
			MediaViewerSettingsView->GetWidget().ToSharedRef()
		];

	TSharedPtr<FMediaImageViewer> FirstImageViewer = InDelegates->GetImageViewer.Execute(EMediaImageViewerPosition::First);
	const bool bHasFirstImageViewer = FirstImageViewer.IsValid() && FirstImageViewer != FNullImageViewer::GetNullImageViewer();

	TSharedPtr<FMediaImageViewer> SecondImageViewer = InDelegates->GetImageViewer.Execute(EMediaImageViewerPosition::Second);
	const bool bHasSecondImageViewer = SecondImageViewer.IsValid() && SecondImageViewer != FNullImageViewer::GetNullImageViewer();

	const float CenterPaddingLeft = bHasFirstImageViewer ? 0.f : 83.f;
	const float CenterPaddingRight = bHasSecondImageViewer ? 0.f : 83.f;

	ChildSlot
	[
		SNew(SBorder)
		.HAlign(HAlign_Fill)
		.BorderImage(FAppStyle::GetBrush("EditorViewportToolBar.Background"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				FirstImageViewer
					? MakeSideToolbar(EMediaImageViewerPosition::First, ToolbarSections::ToolbarLeft)
					: SNullWidget::NullWidget
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(CenterPaddingLeft, 0.f, CenterPaddingRight, 0.f)
			[
				MakeCenterToolbar()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[				
				bHasSecondImageViewer
					? MakeSideToolbar(EMediaImageViewerPosition::Second, ToolbarSections::ToolbarRight)
					: SNullWidget::NullWidget
			]
		]
	];
}

void SMediaViewerToolbar::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged)
{
	const FName MemberPropertyName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(FMediaViewerSettings, ABOrientation))
	{
		Delegates->RefreshView.Execute();
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(FMediaViewerSettings, ABSplitterLocation))
	{
		Delegates->SetABSplitterLocation.Execute(Delegates->GetABSplitterLocation.Execute());
	}
}

TSharedRef<SWidget> SMediaViewerToolbar::MakeCenterToolbar()
{
	using namespace UE::MediaViewer;

	FSlimHorizontalToolBarBuilder ToolbarBuilder(Delegates->GetCommandList.Execute(), FMultiBoxCustomization::None);
	ToolbarBuilder.SetStyle(&FAppStyle::Get(), "ViewportToolbar");
	ToolbarBuilder.SetIsFocusable(false);

	EMediaImageViewerActivePosition ActiveView = Delegates->GetActiveView.Execute();

	const FMediaViewerCommands& Commands = FMediaViewerCommands::Get();

	FSlateIcon ViewIcon;

	if (Delegates->GetActiveView.Execute() != EMediaImageViewerActivePosition::Both)
	{
		ViewIcon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), "EditorViewport.ViewportConfig_OnePane");
	}
	else if (Delegates->GetABOrientation.Execute() == Orient_Horizontal)
	{
		ViewIcon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), "EditorViewport.ViewportConfig_TwoPanesH");
	}
	else
	{
		ViewIcon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), "EditorViewport.ViewportConfig_TwoPanesV");
	}

	ToolbarBuilder.BeginSection(ToolbarSections::ToolbarCenter);
	{
		ToolbarBuilder.BeginBlockGroup();
		{
			FButtonArgs SingleViewArgs;
			SingleViewArgs.ToolTipOverride = LOCTEXT("SetSingleViewToolTip", "View a single image.");
			SingleViewArgs.IconOverride = FSlateIcon(FAppStyle::Get().GetStyleSetName(), "EditorViewport.ViewportConfig_OnePane");
			SingleViewArgs.UserInterfaceActionType = EUserInterfaceActionType::RadioButton;

			SingleViewArgs.Action.ExecuteAction.BindSPLambda(
				this,
				[this]
				{
					Delegates->SetSingleView.Execute();
				}
			);

			SingleViewArgs.Action.GetActionCheckState.BindSPLambda(
				this,
				[this]
				{
					return Delegates->GetActiveView.Execute() == EMediaImageViewerActivePosition::Single
						? ECheckBoxState::Checked
						: ECheckBoxState::Unchecked;
				}
			);

			ToolbarBuilder.AddToolBarButton(SingleViewArgs);

			FButtonArgs ABHorizontalViewArgs;
			ABHorizontalViewArgs.ToolTipOverride = LOCTEXT("SetABHorizontalViewToolTip", "View 2 images side by side.");
			ABHorizontalViewArgs.IconOverride = FSlateIcon(FAppStyle::Get().GetStyleSetName(), "EditorViewport.ViewportConfig_TwoPanesH");
			ABHorizontalViewArgs.UserInterfaceActionType = EUserInterfaceActionType::RadioButton;

			ABHorizontalViewArgs.Action.ExecuteAction.BindSPLambda(
				this,
				[this]
				{
					Delegates->SetABOrientation.Execute(Orient_Horizontal);
				}
			);

			ABHorizontalViewArgs.Action.GetActionCheckState.BindSPLambda(
				this,
				[this]
				{
					return (Delegates->GetActiveView.Execute() == EMediaImageViewerActivePosition::Both
						&& Delegates->GetABOrientation.Execute() == Orient_Horizontal)
						? ECheckBoxState::Checked
						: ECheckBoxState::Unchecked;
				}
			);

			ToolbarBuilder.AddToolBarButton(ABHorizontalViewArgs);

			FButtonArgs ABVerticalViewArgs;
			ABVerticalViewArgs.ToolTipOverride = LOCTEXT("SetABVerticalViewToolTip", "View 2 images, one above the other.");
			ABVerticalViewArgs.IconOverride = FSlateIcon(FAppStyle::Get().GetStyleSetName(), "EditorViewport.ViewportConfig_TwoPanesV");
			ABVerticalViewArgs.UserInterfaceActionType = EUserInterfaceActionType::RadioButton;

			ABVerticalViewArgs.Action.ExecuteAction.BindSPLambda(
				this,
				[this]
				{
					Delegates->SetABOrientation.Execute(Orient_Vertical);
				}
			);

			ABVerticalViewArgs.Action.GetActionCheckState.BindSPLambda(
				this,
				[this]
				{
					return (Delegates->GetActiveView.Execute() == EMediaImageViewerActivePosition::Both
						&& Delegates->GetABOrientation.Execute() == Orient_Vertical)
						? ECheckBoxState::Checked
						: ECheckBoxState::Unchecked;
				}
			);

			ToolbarBuilder.AddToolBarButton(ABVerticalViewArgs);
		}
		ToolbarBuilder.EndBlockGroup();

		ToolbarBuilder.BeginBlockGroup();
		{
			ToolbarBuilder.AddToolBarButton(
				Commands.SwapAB,
				NAME_None,
				FText::GetEmpty(),
				LOCTEXT("SwapABTooltip", "Swaps the image viewers and their offsets."),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "MeshPaint.SwapColor")
			);

			ToolbarBuilder.AddToolBarButton(
				Commands.ToggleLockedTransform,
				NAME_None,
				FText::GetEmpty(),
				LOCTEXT("ToggleLockedTransformToolTip", "Toggle transform lock between viewers."),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericLink")
			);

			ToolbarBuilder.AddToolBarButton(
				Commands.ResetAllTransforms,
				NAME_None,
				FText::GetEmpty(),
				LOCTEXT("ResetTransformToAlToolTip", "Reset the camera transform for all viewers."),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "BlueprintEditor.ResetCamera")
			);
		}
		ToolbarBuilder.EndBlockGroup();
		
		ToolbarBuilder.BeginBlockGroup();
		{
			ToolbarBuilder.AddToolBarButton(
				Commands.Snapshot,
				NAME_None,
				FText::GetEmpty(),
				TAttribute<FText>(),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelViewport.HighResScreenshot")
			);
		}
		ToolbarBuilder.EndBlockGroup();

		ToolbarBuilder.BeginBlockGroup();
		{
			ToolbarBuilder.AddComboButton(
				FUIAction(),
				FOnGetContent::CreateSP(this, &SMediaViewerToolbar::GetBackgroundTextureSettingsWidget),
				FText::GetEmpty(),
				FText::GetEmpty(),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Toolbar.Settings")
			);

			ToolbarBuilder.AddToolBarButton(
				Commands.OpenNewTab,
				NAME_None,
				FText::GetEmpty(),
				TAttribute<FText>(),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.OpenInExternalEditor")
			);
		}
		ToolbarBuilder.EndBlockGroup();
	}
	ToolbarBuilder.EndSection();

	return ToolbarBuilder.MakeWidget();
}

TSharedRef<SWidget> SMediaViewerToolbar::MakeSideToolbar(EMediaImageViewerPosition InPosition, FName InToolbarName)
{
	using namespace UE::MediaViewer;

	TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(InPosition);
	const bool bValidImageViewer = ImageViewer.IsValid() && ImageViewer != FNullImageViewer::GetNullImageViewer();

	TSharedRef<FExtender> Extender = MakeShared<FExtender>();

	if (bValidImageViewer)
	{
		ImageViewer->ExtendToolbar(Extender);
		ImageDetails[static_cast<int32>(InPosition)] = SNew(SBox)
			.Padding(3.f)
			[
				SNew(SMediaImageViewerDetails, InPosition, Delegates.ToSharedRef())
			];
	}

	FSlimHorizontalToolBarBuilder ToolbarBuilder(Delegates->GetCommandListForPosition.Execute(InPosition), FMultiBoxCustomization::None, Extender);
	ToolbarBuilder.SetStyle(&FAppStyle::Get(), "ViewportToolbar");
	ToolbarBuilder.SetIsFocusable(false);

	if (!bValidImageViewer)
	{
		return ToolbarBuilder.MakeWidget();
	}

	ToolbarBuilder.BeginSection(InToolbarName);
	{
		if (ImageViewer.IsValid())
		{
			ToolbarBuilder.BeginBlockGroup();

			{
				ToolbarBuilder.AddComboButton(
					FUIAction(),
					FOnGetContent::CreateSP(this, &SMediaViewerToolbar::GetDetailsWidget, InPosition),
					FText::GetEmpty(),
					FText::GetEmpty(),
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.Tabs.Details")
				);

				ToolbarBuilder.AddComboButton(
					FUIAction(),
					FOnGetContent::CreateSP(this, &SMediaViewerToolbar::MakeScaleMenu, InPosition),
					TAttribute<FText>::CreateSP(this, &SMediaViewerToolbar::GetScaleMenuLabel, InPosition),
					FText::GetEmpty(),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.ScaleGridSnap")
				);
			}

			ToolbarBuilder.EndBlockGroup();
		}
	}
	ToolbarBuilder.EndSection();

	return ToolbarBuilder.MakeWidget();
}

FText SMediaViewerToolbar::GetScaleMenuLabel(EMediaImageViewerPosition InPosition) const
{
	if (TSharedPtr<FMediaImageViewer> ImageViewer = Delegates->GetImageViewer.Execute(InPosition))
	{
		FNumberFormattingOptions FormattingFloat;
		FormattingFloat.SetMaximumFractionalDigits(1);

		return FText::Format(
			LOCTEXT("Scale", "{0}%"),
			FText::AsNumber(ImageViewer->GetPaintSettings().Scale * 100.f, &FormattingFloat)
		);
	}

	return INVTEXT("-");
}

TSharedRef<SWidget> SMediaViewerToolbar::MakeScaleMenu(EMediaImageViewerPosition InPosition) const
{
	TSharedPtr<FUICommandList> CommandList = Delegates->GetCommandListForPosition.Execute(InPosition);

	FMenuBuilder MenuBuilder(true, CommandList, nullptr, false, &FAppStyle::Get(), false);

	if (CommandList.IsValid())
	{
		const FMediaViewerCommands& Commands = FMediaViewerCommands::Get();

		MenuBuilder.AddMenuEntry(Commands.Scale12);
		MenuBuilder.AddMenuEntry(Commands.Scale25);
		MenuBuilder.AddMenuEntry(Commands.Scale50);
		MenuBuilder.AddMenuEntry(Commands.Scale100);
		MenuBuilder.AddMenuEntry(Commands.Scale200);
		MenuBuilder.AddMenuEntry(Commands.Scale400);
		MenuBuilder.AddMenuEntry(Commands.Scale800);
		MenuBuilder.AddMenuEntry(Commands.Scale1000);
		MenuBuilder.AddSeparator();
		MenuBuilder.AddMenuEntry(Commands.ScaleUp);
		MenuBuilder.AddMenuEntry(Commands.ScaleDown);
		MenuBuilder.AddMenuEntry(Commands.ScaleToFit);
		MenuBuilder.AddSeparator();
		MenuBuilder.AddMenuEntry(Commands.ToggleAutoFiltering);
		MenuBuilder.AddMenuEntry(Commands.TogglePixelGrid);
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SMediaViewerToolbar::GetDetailsWidget(EMediaImageViewerPosition InPosition) const
{
	return ImageDetails[static_cast<int32>(InPosition)].ToSharedRef();
}

FReply SMediaViewerToolbar::OnSaveToLibraryClicked()
{
	return FReply::Handled();
}

TSharedRef<SWidget> SMediaViewerToolbar::GetBackgroundTextureSettingsWidget() const
{
	return MediaViewerSettingsWidget.ToSharedRef();
}

} // UE::MediaViewer::Private

#undef LOCTEXT_NAMESPACE
