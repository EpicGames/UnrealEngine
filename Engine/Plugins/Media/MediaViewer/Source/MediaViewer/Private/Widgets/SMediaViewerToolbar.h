// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "SViewportToolBar.h"

#include "MediaViewerDelegates.h"

class IStructureDetailsView;

namespace UE::MediaViewer
{
	class FMediaViewer;
	struct FMediaViewerDelegates;
}

namespace UE::MediaViewer::Private
{

/**
 * Toolbar for the Media Viewer.
 */
class SMediaViewerToolbar : public SViewportToolBar, public FNotifyHook
{
	SLATE_DECLARE_WIDGET(SMediaViewerToolbar, SCompoundWidget)

	SLATE_BEGIN_ARGS(SMediaViewerToolbar)
		{}
	SLATE_END_ARGS()

public:
	SMediaViewerToolbar();

	virtual ~SMediaViewerToolbar() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<FMediaViewerDelegates>& InDelegates);

	//~ Begin FNotifyHook
	virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged) override;
	//~ End FNotifyHook

protected:
	TSharedPtr<FMediaViewerDelegates> Delegates;
	TSharedPtr<SWidget> ImageDetails[static_cast<int32>(EMediaImageViewerPosition::COUNT)];
	TSharedPtr<IStructureDetailsView> MediaViewerSettingsView;
	TSharedPtr<SWidget> MediaViewerSettingsWidget;

	TSharedRef<SWidget> MakeSideToolbar(EMediaImageViewerPosition InPosition, FName InToolbarName);

	TSharedRef<SWidget> MakeCenterToolbar();

	TSharedRef<SWidget> GetBackgroundTextureSettingsWidget() const;

	FText GetScaleMenuLabel(EMediaImageViewerPosition InPosition) const;
	TSharedRef<SWidget> MakeScaleMenu(EMediaImageViewerPosition InPosition) const;
	
	TSharedRef<SWidget> GetDetailsWidget(EMediaImageViewerPosition InPosition) const;

	FReply OnSaveToLibraryClicked();
};

} // UE::MediaViewer::Private
