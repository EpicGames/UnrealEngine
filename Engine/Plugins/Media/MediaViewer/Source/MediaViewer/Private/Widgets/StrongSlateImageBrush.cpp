// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/StrongSlateImageBrush.h"

FStrongSlateImageBrush::FStrongSlateImageBrush(UObject* InResourceObject, const UE::Slate::FDeprecateVector2DParameter& InImageSize, 
	const FSlateColor& InTint, ESlateBrushTileType::Type InTiling, ESlateBrushImageType::Type InImageType)
	: FSlateImageBrush(InResourceObject, InImageSize, InTint, InTiling, InImageType)
	, ResourceObjectPtr(InResourceObject)
{
}

FString FStrongSlateImageBrush::GetReferencerName() const
{
	static const FString ReferencerName = TEXT("StrongSlateImageBrush");
	return ReferencerName;
}

void FStrongSlateImageBrush::AddReferencedObjects(FReferenceCollector& InCollector)
{
	if (ResourceObjectPtr)
	{
		InCollector.AddReferencedObject(ResourceObjectPtr);
	}
}
