// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Brushes/SlateImageBrush.h"
#include "UObject/GCObject.h"

/**
 * Ignores the Margin. Just renders the image. Can tile the image instead of stretching.
 */
struct FStrongSlateImageBrush : public FSlateImageBrush, public FGCObject
{
	FStrongSlateImageBrush(UObject* InResourceObject, const UE::Slate::FDeprecateVector2DParameter& InImageSize, 
		const FSlateColor& InTint = FSlateColor(FLinearColor(1, 1, 1, 1)), 
		ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile, 
		ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor);

	//~ Begin FGCObject
	virtual FString GetReferencerName() const override;
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	//~ End FGCObject

protected:
	TObjectPtr<UObject> ResourceObjectPtr;
};
