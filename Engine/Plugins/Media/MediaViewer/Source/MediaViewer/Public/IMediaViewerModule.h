// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "ImageViewer/IMediaImageViewerFactory.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "Widgets/IMediaViewerLibraryWidget.h"

class SDockTab;
struct FAssetData;
struct FMediaViewerLibraryItem;

namespace UE::MediaViewer
{

class FMediaImageViewer;
class IMediaViewerLibrary;
class SMediaViewerTab;
enum class EMediaImageViewerPosition : uint8;
struct FMediaViewerArgs;

/**
 * Media Viewer - Display and compare media
 */
class IMediaViewerModule : public IModuleInterface
{
public:
	static IMediaViewerModule& Get()
	{
		return FModuleManager::GetModuleChecked<IMediaViewerModule>("MediaViewer");
	}

	/**
	 * Image viewer factory methods.
	 */

	virtual bool IsFactoryRegistered(FName InFactoryName) const = 0;
	virtual bool HasFactoryFor(const FAssetData& InAssetData) const = 0;
	virtual bool HasFactoryFor(UObject* InObject) const = 0;
	virtual void RegisterFactory(FName InFactoryName, const TSharedRef<IMediaImageViewerFactory>& InFactory) = 0;
	virtual void UnregisterFactory(FName InFactoryName) = 0;

	/**
	 * Interacting with the default tab.
	 */

	/** Will open the tab if necessary. Will give the opened tab focus. */
	virtual bool OpenTab() = 0;

	/** If the settings differ from the currently opened tab's settings, it will regenerate the entire display. */
	virtual bool OpenTab(const FMediaViewerArgs& InMediaViewerArgs) = 0;

	/** Returns the Library, which is common between all tabs. */
	virtual TSharedRef<IMediaViewerLibrary> GetLibrary() const = 0;

	virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(const FAssetData& InAssetData) const = 0;
	virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(TNotNull<UObject*> InObject) const = 0;
	virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(FName InItemType, const FMediaViewerLibraryItem& InSavedItem) const = 0;

	virtual bool HasImage(EMediaImageViewerPosition InPosition) const = 0;
	virtual TSharedPtr<FMediaImageViewer> GetImage(EMediaImageViewerPosition InPosition) const = 0;

	/** 
	 * If the widget is open, these will control the displayed images programmatically. 
	 * Returns true if the widget is open and a generator is found.
	 */
	virtual bool SetImage(EMediaImageViewerPosition InPosition, const FAssetData& InAssetData) = 0;
	virtual bool SetImage(EMediaImageViewerPosition InPosition, UObject* InObject) = 0;
	virtual bool SetImage(EMediaImageViewerPosition InPosition, const TSharedRef<FMediaImageViewer> InImageViewer) = 0;
	virtual bool ClearImage(EMediaImageViewerPosition InPosition) = 0;

	/**
	 * Methods for creating custom implementations
	 */

	/** Create a raw copy of the reference viewer. */
	virtual TSharedRef<SMediaViewerTab> CreateMediaViewer(const FMediaViewerArgs& InMediaViewerArgs) = 0;

	/** Creates a Library widget. */
	virtual TSharedRef<IMediaViewerLibraryWidget> CreateLibrary(const IMediaViewerLibraryWidget::FArgs& InArgs) = 0;

	/**
	 * Methods for creating new standalone tabs
	 */
	virtual TSharedPtr<SDockTab> OpenStandaloneTab() = 0;
};

} // UE::MediaViewer
