// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/NotNull.h"
#include "Templates/SharedPointerFwd.h"

class UObject;
struct FAssetData;
struct FMediaViewerLibraryItem;

namespace UE::MediaViewer
{

class FMediaImageViewer;

struct IMediaImageViewerFactory
{
	/** Priorities should be in the range of 1000 (first) to 10000 (last). */
	int32 Priority = 5000;

	/** Whether this factory supports this asset type. */
	virtual bool SupportsAsset(const FAssetData& InAssetData) const = 0;
	/** Creates an image viewer based on this asset. */
	virtual TSharedPtr<FMediaImageViewer> CreateImageViewer(const FAssetData& InAssetData) const = 0;
	/** Creates a library item based on this asset. */
	virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(const FAssetData& InAssetData) const = 0;

	/** Whether this factory supports this object. */
	virtual bool SupportsObject(TNotNull<UObject*> InObject) const = 0;
	/** Create an image viewer based on this object. */
	virtual TSharedPtr<FMediaImageViewer> CreateImageViewer(TNotNull<UObject*> InObject) const = 0;
	/** Create a library item based on this object. */
	virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(TNotNull<UObject*> InObject) const = 0;

	/** Whether this item type is supported (E.g. "Texture2D" or "Color"). */
	virtual bool SupportsItemType(FName InItemType) const = 0;
	/** Recreate this library item as the correct subclass. */
	virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem(const FMediaViewerLibraryItem& InSavedItem) const = 0;
};

} // UE::MediaViewer
