// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "Templates/SharedPointer.h"

#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Layout/PaintGeometry.h"
#include "Math/Color.h"
#include "Math/Range.h"
#include "Math/Rotator.h"
#include "Math/Vector.h"
#include "MediaViewer.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Rendering/RenderingCommon.h"
#include "UObject/GCObject.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/StructOnScope.h"

#include "MediaImageViewer.generated.h"

class FExtender;
class FPaintArgs;
class FSlateClippingZone;
class FSlateRect;
class FSlateWindowElementList;
class FStructOnScope;
class FUICommandList;
class FWidgetStyle;
class SWidget;
class UCanvas;
class UMaterialInstanceDynamic;
class UObject;
class UTexture;
class UTextureRenderTarget2D;
struct FFloatRange;
struct FGeometry;
struct FInputEventState;
struct FInputKeyEventArgs;
struct FMediaViewerLibraryItem;
struct FSlateBrush;
struct FSlateImageBrush;
template <typename T, typename... Ts> class TVariant;

/**
 * Settings related to drawing panel contents other than the image itself.
 */
USTRUCT()
struct FMediaImageViewerPanelSettings
{
	GENERATED_BODY()

	/**
	 * Color for the background within the image rectangle.
	 * If this is different to the clear color, it shows where the image is even
	 * if nothing is drawn.
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Panel")
	TOptional<FLinearColor> BackgroundColor;

	/**
	 * If set, this is drawn over the top of the background color.
	 * Example usage: checkered background.
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Panel")
	TSoftObjectPtr<UTexture> BackgroundTexture;
};

UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EMediaImageMirrorPlane : uint8
{
	None       = 0      UMETA(Hidden),
	Horizontal = 1 << 0,
	Vertical   = 1 << 1,
	Depth      = 1 << 2 UMETA(ToolTip = "3D Only"), // Front to back flipping

	MAX        = Depth UMETA(Hidden),
	TwoDim     = Horizontal | Vertical UMETA(Hidden),
	ThreeDim   = TwoDim | Depth UMETA(Hidden)
};
ENUM_CLASS_FLAGS(EMediaImageMirrorPlane)

USTRUCT()
struct FMediaImagePaintSettings
{
	GENERATED_BODY()

	/**
	 * Offset from the origin (center) of the viewer.
	 * X: Horizontal (L->R)
	 * Y: Vertical (T->B)
	 * Z: Depth (F->B)
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Media")
	FVector Offset = FVector::ZeroVector;

	/** Rotation about the offset, for 3d objects. */
	UPROPERTY(EditAnywhere, Config, Category = "Media")
	FRotator Rotation = FRotator::ZeroRotator;

	/**
	 * The scale of the image.
	 * A scale of 2 will mean the image is twice as big.
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Media")
	float Scale = 1.f;

	/**
	 * Image mirroring along the center axes
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Media", meta = (Bitmask, BitmaskEnum = "/Script/MediaViewer.EMediaImageMirrorPlane"))
	uint8 MirrorPlanes = 0;

	/**
	 * When true, bilinear filtering is automatically disabled (nearest-neighbor sampling)
	 * once the zoom level reaches AutoFilteringThreshold. This allows individual pixels to be
	 * seen clearly when zoomed in far enough.
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Media")
	bool bAutoFilteringEnabled = true;

	/**
	 * The zoom scale at which filtering is automatically disabled when bAutoFilteringEnabled is true.
	 * A value of 2.0 means filtering is disabled at 200% zoom and above.
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Media",
		meta = (EditCondition = "bAutoFilteringEnabled", UIMin = 1.0, ClampMin = 1.0, UIMax = 8.0, ClampMax = 8.0, Units = "x"))
	float AutoFilteringThreshold = 4.0f;

	/**
	 * When true, a pixel grid overlay is drawn when zoomed in with nearest-neighbor filtering active,
	 * making individual pixel boundaries clearly visible.
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Media")
	bool bPixelGridEnabled = true;

	/**
	 * Minimum screen-pixel size a source pixel must reach before the grid is drawn.
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Media",
		meta = (EditCondition = "bPixelGridEnabled", UIMin = 8.0, ClampMin = 1.0, UIMax = 128.0, ClampMax = 128.0))
	float PixelGridThreshold = 9.f;

	/** Whether to enable the MipOverride */
	UPROPERTY(EditAnywhere, Config, Category = "Media", meta = (InlineEditConditionToggle))
	bool bEnableMipOverride = false;

	/**
	 * The requested mip level. If the mip isn't available, the highest available value will be used.
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Media", meta = (EditCondition = "bEnableMipOverride", UIMin = 0, ClampMin = 0, UIMax = 10, ClampMax = 10))
	uint8 MipOverride = 0;

	/**
	 * Color to tint the image brush with.
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Media")
	FLinearColor Tint = FLinearColor::White;

	/** Returns a Mip value, if enabled. */
	TOptional<uint8> GetMipOverride() const
	{
		if (bEnableMipOverride)
		{
			return MipOverride;
		}

		return {};
	}

	/** Always returns a Mip value, even if unset, will default to 0. */
	uint8 GetMipLevel() const
	{
		return GetMipOverride().Get(0);
	}
};

namespace UE::MediaViewer
{

class FMediaImageStatusBarExtender;
struct FMediaViewerDelegates;
	
/** Information about the image viewer's image. */
struct FMediaImageViewerInfo
{
	/** Unique Id for the *source*. */
	FGuid Id;
	/** Size of the image to draw. */
	FIntPoint Size;
	/** The number of mips the image has. */
	int32 MipCount;
	/** The name of the image. */
	FText DisplayName;
};

/**
 * Parameters used by the image viewers to paint their image.
 */
struct FMediaImageSlatePaintParams
{
	/** Slate paint args. */
	const FPaintArgs& Args;
	/** Slate geometry for the panel. */
	const FGeometry& AllottedGeometry;
	/** Culling rect for the panel. */
	const FSlateRect& MyCullingRect;
	/** Slate for the panel. */
	const FWidgetStyle& WidgetStyle;
	/** Whether the parent painted widget was enabled. */
	const bool bParentEnabled;
	/** UV Range to draw the image on, either horizontally (L->R) and vertically (T->B). */
	const FFloatRange& UVRange;
	/** Scale of the panel. */
	const float DPIScale;
	/** Orientation of the panel split. */
	const EOrientation Orientation;
	/** Size of the panel in the window. */
	const FVector2D ViewerSize;
	/** Position of the panel in the window. */
	const FVector2D ViewerPosition;
	/** Opacity to draw the image with. 0-1. */
	float ImageOpacity;
	/** Layer Id to draw the image onto. Increment it if you draw anything. */
	int32 LayerId;
	/** List of elements to draw into the layer. */
	FSlateWindowElementList& DrawElements;
};

/** Geometry of the image within the panel. */
struct FMediaImageSlatePaintGeometry
{
	/** Position of the image within the painted area. */
	const FVector2D Position;
	/** Size of the image within the painted area. */
	const FVector2D Size;
	/** Slate geometry of the image within the painted area. */
	const FPaintGeometry Geometry;
};

/**
 * Parameters used by the image viewers to paint their image.
 */
struct FMediaImageCanvasPaintParams
{
	/* The canvas to draw on. */
	UCanvas* Canvas;
	/** UV Range to draw the image on, either horizontally (L->R) and vertically (T->B). */
	const FFloatRange& UVRange;
	/** Orientation of the panel split. */
	const EOrientation Orientation;
	/** Opacity to draw the image with. 0-1. */
	float ImageOpacity;
};

/** Geometry of the image within the panel. */
struct FMediaImageCanvasPaintGeometry
{
	/** Position of the image within the painted area. */
	const FVector2D Position;
	/** Size of the image within the painted area. */
	const FVector2D Size;
};

/**
 * Image resource for the AB Image Viewer representing a source that can be displayed as a 2d image.
 */
class FMediaImageViewer : public FNotifyHook, public FGCObject, public TSharedFromThis<FMediaImageViewer>
{
protected:
	struct FPrivateToken {};

public:
	/**
	 * Given an object, tries to fetch a nice display name for it.
	 * If it's contained within an actor, it will return the actor's label / name.
	 * Otherwise it returns the object's name.
	 */
	MEDIAVIEWER_API static FText GetObjectDisplayName(const UObject* InObject);

	MEDIAVIEWER_API FMediaImageViewer(const FMediaImageViewerInfo& InImageInfo);

	virtual ~FMediaImageViewer() override = default;

	const FMediaImageViewerInfo& GetInfo() const { return ImageInfo; }

	virtual bool IsValid() const { return ImageInfo.Id.IsValid(); }

	virtual bool SupportsMip() const { return true; }
	
	/** 
	 * Called, for instance, when an image is dropped into the viewer and it creates an image viewer,
	 * but the image already exists in the library. This image viewer should have its id updated to 
	 * reflect the id already in the library.
	 */
	void UpdateId(const FGuid& InId);

	virtual TSharedPtr<FMediaViewerLibraryItem> CreateLibraryItem() const = 0;

	const FMediaImageViewerPanelSettings& GetPanelSettings() const { return PanelSettings; }
	FMediaImageViewerPanelSettings& GetPanelSettings() { return PanelSettings; }

	const FMediaImagePaintSettings& GetPaintSettings() const { return PaintSettings; }
	FMediaImagePaintSettings& GetPaintSettings() { return PaintSettings; }

	void InitBackgroundTexture();

	virtual TOptional<TVariant<FColor, FLinearColor>> GetPixelColor(const FIntPoint& InPixelCoords, int32 InMipLevel) const = 0;

	/** Paint the panel and image on to the viewer. */
	void Paint(FMediaImageSlatePaintParams& InPaintParams);

	/** Paint the panel and image to a canvas. */
	void Paint(const FMediaImageCanvasPaintParams& InPaintParams);

	/**
	 * Extend the tool bar (top). @see MediaViewer.h
	 * @param InOutToolbarExtender The extender to populate.
	 */
	virtual void ExtendToolbar(const TSharedRef<FExtender>& InOutToolbarExtender) {}

	/**
	 * Extend the status bar (top over overlay). @see MediaViewer.h
	 * @param InOutStatusBarExtender - The extender to populate.
	 */
	virtual void ExtendStatusBar(FMediaImageStatusBarExtender& InOutStatusBarExtender) {}

	/**
	 * Returns a widget to place on the overlay.
	 * If the widget is on the A position, it should be left aligned.
	 * If the widget is in the B position, it should be right aligned.
	 * The above alignment will be used for the widget's slot.
	 * If no overlay is required, return nullptr.
	 */
	virtual TSharedPtr<SWidget> GetOverlayWidget(EMediaImageViewerPosition InPosition, const TSharedPtr<FMediaViewerDelegates>& InDelegates) { return nullptr; }

	/** 
	 * Responds to tracking started in this viewer's viewport. 
	 * 
	 * @param   InputState - The state of the input when tracking started.
	 * @param   MousePosition - The position of the mouse when tracking started.
	 * @returns True if custom tracking was started.
	 */
	virtual bool OnTrackingStarted(const FInputEventState& InInputState, const FIntPoint& InMousePosition) { return false; }

	/** 
	 * Responds to tracking stopped in this viewer's viewport. 
	 *
	 * @param   MousePosition - The position of the mouse when tracking started.
	 */
	virtual void OnTrackingStopped(const FIntPoint& InMousePosition) {}

	/**
	 * Responds to mouse movement in this viewer's viewport.
	 *
	 * @param   MousePosition - The position of the mouse when tracking stoppped.
	 * @returns True if custom tracking was stopped.
	 */
	virtual void OnMouseMove(const FVector2D& InMousePosition) {}

	/**
	 * Responds to key input in this image's viewport.
	 * Note: this is after the ImageWidgets internal event consumption.
	 *
	 * @param   InputState - The state of the input when tracking started.
	 * @returns True if the input was consumed.
	 */
	virtual bool OnKeyPressed(const FInputKeyEventArgs& InEventArgs) { return false; }

	/**
	 * Returns a custom struct on scope to display media-specific settings to the user.
	 */
	virtual TSharedPtr<FStructOnScope> GetCustomSettingsOnScope() const { return nullptr; }

	/** Returns true if the image is mirrored on the given axis. */
	bool IsMirrored(EMediaImageMirrorPlane InPlane) const;

	/** Use to set mirror flag, don't set directly! */
	void SetMirrored(EMediaImageMirrorPlane InPlane, bool bInSet);

	/** Triggered when a shape is mirrored. */
	virtual void OnMirrored() {};

	/** Returns the center of the viewer, not necessarily the center of the splitter it's contained within. */
	MEDIAVIEWER_API FVector2D GetViewerCenter(const FVector2D& InViewerSize) const;

	/** Returns the location of the top-left corner of the image for it to be painted in the centre of the viewer. */
	MEDIAVIEWER_API FVector2D GetPaintOffsetForViewerCenter(const FVector2D& InViewerSize) const;

	/** Applies paint settings to the default center position. */
	MEDIAVIEWER_API FVector2D GetPaintOffset(const FVector2D& InViewerSize, const FVector2D& InViewerPosition) const;

	/** Returns the size of the image, including things like scale. */
	MEDIAVIEWER_API FVector2D GetPaintSize() const;

	/** Called when a potential outer has been removed. Returns true if this is no longer valid. */
	virtual bool OnOwnerCleanup(UObject* InObject) 
	{
		return false;
	}

	//~ Begin FNotifyHook
	virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged) override;
	//~ End FNotifyHook

	//~ Begin FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	//~ End FGCObject

protected:
	static const FSlateColorBrush BackgroundColorBrush;

	FMediaImageViewerInfo ImageInfo;
	FMediaImageViewerPanelSettings PanelSettings;
	FMediaImagePaintSettings PaintSettings;
	FSlateImageBrush BackgroundImageBrush;
	TSharedPtr<FSlateBrush> Brush;
	ESlateDrawEffect DrawEffects = ESlateDrawEffect::NoPixelSnapping;
	TStrongObjectPtr<UMaterialInstanceDynamic> MipAdjustedMaterial;
	TStrongObjectPtr<UTextureRenderTarget2D> MipAdjustedRenderTarget;
	TSharedPtr<FSlateBrush> MipAdjustedBrush;
	bool bInvalidatedMipTexture = true;

	/**
	 * Refreshes MipAdjustedRenderTarget / MipAdjustedBrush for the current mip level.
	 * Returns true if the brush is drawable this frame. False is "skip this frame" - it covers
	 * transient cases like the mip material's shaders still compiling, where bInvalidatedMipTexture
	 * stays true so the next paint retries. Callers should fall back to plain Brush, not treat
	 * false as fatal.
	 */
	bool UpdateMipBrush();

	FSlateClippingZone CreateSlateClippingZone(const FSlateRect& InCullingRect, float InDPIScale, const FVector2D& InViewerPosition, 
		EOrientation InOrientation, const FFloatRange& InUVRange) const;

	MEDIAVIEWER_API virtual void PaintPanel(FMediaImageSlatePaintParams& InPaintParams, const FMediaImageSlatePaintGeometry& InPaintGeometry);

	MEDIAVIEWER_API virtual void PaintImage(FMediaImageSlatePaintParams& InPaintParams, const FMediaImageSlatePaintGeometry& InPaintGeometry);

	/** Draws a pixel boundary grid over the image when nearest-neighbor filtering is active. */
	MEDIAVIEWER_API virtual void PaintPixelGrid(FMediaImageSlatePaintParams& InPaintParams, const FMediaImageSlatePaintGeometry& InPaintGeometry);

	MEDIAVIEWER_API virtual void PaintPanel(const FMediaImageCanvasPaintParams& InPaintParams, const FMediaImageCanvasPaintGeometry& InPaintGeometry);

	MEDIAVIEWER_API virtual void PaintImage(const FMediaImageCanvasPaintParams& InPaintParams, const FMediaImageCanvasPaintGeometry& InPaintGeometry);
};

} // UE::MediaViewer
