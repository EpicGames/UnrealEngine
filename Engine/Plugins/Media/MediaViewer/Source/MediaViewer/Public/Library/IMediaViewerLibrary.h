// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Internationalization/Text.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointerFwd.h"

struct FMediaViewerLibraryGroup;
struct FMediaViewerLibraryItem;

namespace UE::MediaViewer
{

/**
 * Groups are identified by a single GUID.
 * Items are identified by a single GUID.
 * Library Items associate an item's GUID with a position in the Library. They are uniquely 
 *   identified by an item GUID and a group GUID.
 */
class IMediaViewerLibrary
{
public:
	struct FGroupItem
	{
		FGuid GroupId;
		FGuid ItemId;

		bool operator==(const FGroupItem& InOther) const
		{
			return GroupId == InOther.GroupId
				&& ItemId == InOther.ItemId;
		}

		friend int32 GetTypeHash(const IMediaViewerLibrary::FGroupItem& InGroupItem)
		{
			return HashCombineFast(
				GetTypeHash(InGroupItem.GroupId),
				GetTypeHash(InGroupItem.ItemId)
			);
		}
	};

	enum class EChangeType : uint8
	{
		GroupAdded,
		GroupRemoved,
		GroupItemsChanged,
		ItemAdded,
		ItemRemoved,
		ItemGroupChanged
	};

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnChanged, TSharedRef<IMediaViewerLibrary> /* This */, EChangeType)

	/**
	 * @returns The id of the default group.
	 */
	virtual const FGuid& GetDefaultGroupId() const = 0;

	/**
	 * @returns The id of the history group.
	 */
	virtual const FGuid& GetHistoryGroupId() const = 0;

	/**
	 * @returns The id of the snapshot group.
	 */
	virtual const FGuid& GetSnapshotsGroupId() const = 0;

	/**
	 * Return the list of groups.
	 */
	 virtual const TArray<TSharedRef<const FMediaViewerLibraryGroup>>& GetGroups() const = 0;

	/**
	 * Add a custom group to the Library.
	 * @param InNewGroup Group to be added. Id must be unique.
	 * @returns False if a group with the given name already exits, otherwise true.
	 */
	virtual bool AddGroup(const TSharedRef<FMediaViewerLibraryGroup>& InNewGroup) = 0;

	/**
	 * Return the group with the given id. Do not change guid or items manually.
	 * @param InGroupId The unique if of the group
	 * @returns An nullptr if the given id is invalid, otherwise the id of the group at that position.
	 */
	virtual TSharedPtr<FMediaViewerLibraryGroup> GetGroup(const FGuid& InGroupId) const = 0;

	/**
	 * Removes all items from a group. 
	 * @returns The number of items removed.
	 */
	virtual int32 EmptyGroup(const FGuid& InGroupId) = 0;

	/**
	 * Scans the group and removes invalid entries.
	 */
	virtual void RemoveInvalidGroupItems(const FGuid& InGroup) = 0;

	/**
	 * Returns true if the group with this id can be removed.
	 */
	virtual bool CanRemoveGroup(const FGuid& InGroupIdToRemove) const = 0;

	/**
	 * Remove an existing group and any items in the group.
	 * @param InGroupIdToRemove Unique identifier of the group. Must not be the default group name.
	 * @returns The group that was removed, including all its items.
	 */
	virtual TSharedPtr<FMediaViewerLibraryGroup> RemoveGroup(const FGuid& InGroupIdToRemove) = 0;

	/**
	 * Returns the first item that matches the item type and string value.
	 */
	virtual TSharedPtr<FMediaViewerLibraryItem> FindItemByValue(FName InItemType, const FString& InStringValue) const = 0;

	/**
	 * Retrieve the existing item for a given unique identifier. Do not change guid or group manually.
	 * @param InItemId The unique identifier of the item.
	 * @return The pointer to the item or an invalid pointer if no item with the given unique identifier exists.
	 */
	virtual TSharedPtr<FMediaViewerLibraryItem> GetItem(const FGuid& InItemId) const = 0;

	/**
	 * Finds the first group containing the given Item Id.
	 * @param InItemId The unique identifier of the item.
	 * @return The group containing the item or null.
	 */
	virtual TSharedPtr<FMediaViewerLibraryGroup> GetItemGroup(const FGuid& InItemId) const = 0;

	/**
	 * Adds an item to the Library without place it in a group
	 * @param InNewItem Data for the item that is being added.
	 */
	virtual bool AddItem(const TSharedRef<FMediaViewerLibraryItem>& InNewItem) = 0;

	/**
	 * Add an item to a group.
	 * @param InNewItem Data for the item that is being added.
	 * @param InTargetGroupId Id of the group. Default group is used if the id is empty.
	 * @param InIndex Index in the array of items in the group.
	 * @returns True if the item was added successfully. Will fail if an invalid group id is supplied or the item already exists.
	 */
	virtual bool AddItemToGroup(const TSharedRef<FMediaViewerLibraryItem>& InNewItem, TOptional<FGuid> InTargetGroupId = {},
		int32 InIndex = INDEX_NONE) = 0;

	/**
	 * Add an item below another item.
	 * @param InNewItem New item that is being added.
	 * @param InTargetItem Group/Item to add below.
	 * @returns True if the item was added successfully. Will fail if an invalid group id is supplied or the item already exists.
	 */
	virtual bool AddItemBelowItem(const TSharedRef<FMediaViewerLibraryItem>& InNewItem, const FGroupItem& InTargetItem) = 0;

	/**
	 * Move an already existing to another group, adding it to the end of the group.
	 * @param InItemToMove The existing group/item that should be moved.
	 * @param InTargetGroupId Unique identifier of the group it should be moved into.
	 * @param InIndex Index in the array of items in the group.
	 * @returns False if the the item or group don't exist, or the item is already in that group.
	 */
	virtual bool MoveItemToGroup(const FGroupItem& InItemToMove, const FGuid& InTargetGroupId, int32 InIndex = INDEX_NONE) = 0;

	/**
	 * Move an already existing to another place inside its group.
	 * @param InItemToMove The existing group/item that should be moved.
	 * @param InIndex Index in the array of items in the group.
	 * @returns False if the the item doesn't exist, or the item is already at that index.
	 */
	virtual bool MoveItemWithinGroup(const FGroupItem& InItemToMove, int32 InIndex) = 0;

	/**
	 * Move an already existing item below another, potentially changing its group.
	 * @param InItemToMove The existing group/item that should be moved.
	 * @param InTargetItem The group/item below which the item should be moved.
	 * @returns False if the either item doesn't exist. Otherwise true.
	 */
	virtual bool MoveItemBelowItem(const FGroupItem& InItemToMove, const FGroupItem& InTargetItem) = 0;

	/**
	 * Returns true if this item can be removed from the group.
	 */
	virtual bool CanRemoveItemFromGroup(const FGroupItem& InItemToRemove) const = 0;

	/**
	 * Return true if the item was removed from the group.
	 */
	virtual bool RemoveItemFromGroup(const FGroupItem& InItemToRemove) = 0;

	/**
	 * Returns true if this item can be removed from every group.
	 */
	virtual bool CanRemoveItem(const FGuid& InItemIdToRemove) const = 0;

	/**
	 * Return true if the item was removed from every group.
	 */
	virtual TSharedPtr<FMediaViewerLibraryItem> RemoveItem(const FGuid& InItemIdToRemove) = 0;

	/**
	 * When a changed is triggered, this event is triggered.
	 */
	virtual FOnChanged::RegistrationType& GetOnChanged() = 0;
};

} // UE::MediaViewer
