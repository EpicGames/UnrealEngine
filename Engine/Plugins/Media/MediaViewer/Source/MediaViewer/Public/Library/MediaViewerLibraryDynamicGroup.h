// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Library/MediaViewerLibraryGroup.h"

#include "Delegates/Delegate.h"
#include "Misc/Optional.h"

namespace UE::MediaViewer
{

class IMediaViewerLibrary;

/**
 * Contains all data for a Library group which has dynamic elements based on an external source.
*/
struct FMediaViewerLibraryDynamicGroup : public FMediaViewerLibraryGroup
{
	DECLARE_DELEGATE_RetVal(TArray<TSharedRef<FMediaViewerLibraryItem>>, FGenerateItems)

	MEDIAVIEWER_API FMediaViewerLibraryDynamicGroup(const TSharedRef<IMediaViewerLibrary>& InLibrary, const FText& InName,
		const FText& InToolTip, FGenerateItems InItemGenerator);

	MEDIAVIEWER_API FMediaViewerLibraryDynamicGroup(const TSharedRef<IMediaViewerLibrary>& InLibrary, const FGuid& InId,
		const FText& InName, const FText& InToolTip, FGenerateItems InItemGenerator);

	virtual ~FMediaViewerLibraryDynamicGroup() override = default;

	MEDIAVIEWER_API virtual void UpdateItems();

	MEDIAVIEWER_API FSimpleMulticastDelegate::RegistrationType& GetOnItemsUpdated();

protected:
	TWeakPtr<IMediaViewerLibrary> LibraryWeak;
	FGenerateItems GenerateItemsDelegate;
	FSimpleMulticastDelegate OnItemsUpdated;

	TOptional<TArray<FGuid>> GetUpdatedIs(const TArray<FGuid>& InCurrentIds);
};

} // UE::MediaViewer
