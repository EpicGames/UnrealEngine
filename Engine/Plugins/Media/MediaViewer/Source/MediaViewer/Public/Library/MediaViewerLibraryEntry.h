// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Misc/Guid.h"

#include "MediaViewerLibraryEntry.generated.h"

namespace UE::MediaViewer
{

enum class EMediaViewerLibraryEntryType
{
	Invalid,
	Group,
	Item
};

} // UE::MediaViewer

/**
 * Contains all data for a Library group
*/
USTRUCT()
struct FMediaViewerLibraryEntry
{	
	GENERATED_BODY()

	/** Name of this entry. */
	UPROPERTY()
	FText Name;

	/** Tooltip that is shown when hovering over any part of the item's widget in the Library. */
	UPROPERTY()
	FText ToolTip;

	FMediaViewerLibraryEntry();
	MEDIAVIEWER_API FMediaViewerLibraryEntry(const FText& InName, const FText& InToolTip);
	MEDIAVIEWER_API FMediaViewerLibraryEntry(const FGuid& InId, const FText& InName, const FText& InToolTip);

	virtual ~FMediaViewerLibraryEntry() = default;

	const FGuid& GetId() const;

	void InvalidateId();

	virtual UE::MediaViewer::EMediaViewerLibraryEntryType GetEntryType() const { return UE::MediaViewer::EMediaViewerLibraryEntryType::Invalid; }

	bool operator==(const FMediaViewerLibraryEntry& InOther) const
	{
		return GetEntryType() == InOther.GetEntryType()
			&& Id == InOther.GetId();
	}

protected:
	/** Unique id for this entry. */
	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	FGuid Id;
};

inline int32 GetTypeHash(const FMediaViewerLibraryEntry& InEntry)
{
	return GetTypeHash(InEntry.GetId());
}
