// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Library/MediaViewerLibraryEntry.h"

#include "Containers/Array.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

#include "MediaViewerLibraryGroup.generated.h"

class UMediaViewerLibraryIni;
struct FMediaViewerLibraryItem;

/**
 * Contains all data for a Library group
*/
USTRUCT()
struct FMediaViewerLibraryGroup : public FMediaViewerLibraryEntry
{
	friend class ::UMediaViewerLibraryIni;

protected:
	struct FPrivateToken {};

public:
	GENERATED_BODY()

	FMediaViewerLibraryGroup();
	MEDIAVIEWER_API FMediaViewerLibraryGroup(const FText& InName, const FText& InToolTip, bool bDynamic);
	MEDIAVIEWER_API FMediaViewerLibraryGroup(const FGuid& InId, const FText& InName, const FText& InToolTip, bool bDynamic);
	MEDIAVIEWER_API FMediaViewerLibraryGroup(FPrivateToken&& InPrivateToken, const FMediaViewerLibraryGroup& InSavedGroup);

	virtual ~FMediaViewerLibraryGroup() override = default;

	MEDIAVIEWER_API const TArray<FGuid>& GetItems() const;

	/** Returns the index the item was added to in the list. */
	MEDIAVIEWER_API int32 AddItem(const FGuid& InItemUd, int32 InIndex = INDEX_NONE);

	MEDIAVIEWER_API int32 FindItemIndex(const FGuid& InItemId) const;

	MEDIAVIEWER_API bool ContainsItem(const FGuid& InItemId) const;

	MEDIAVIEWER_API bool RemoveItem(const FGuid& InItemId);
	
	MEDIAVIEWER_API bool RemoveItemAt(int32 InIndex);

	/** Returns the number of elements removed. */
	MEDIAVIEWER_API int32 Empty();

	/** 
	 * If true the group's contents are generated dynamically and not saved.
	 * Its contents can be saved to the history group or copied to other groups.
	 */
	bool IsDynamic() const;

	//~ Begin FMediaViewerLibraryEntry
	virtual UE::MediaViewer::EMediaViewerLibraryEntryType GetEntryType() const { return UE::MediaViewer::EMediaViewerLibraryEntryType::Group; }
	//~ End FMediaViewerLibraryEntry

protected:
	bool bDynamic;

	UPROPERTY()
	TArray<FGuid> Items;
};
