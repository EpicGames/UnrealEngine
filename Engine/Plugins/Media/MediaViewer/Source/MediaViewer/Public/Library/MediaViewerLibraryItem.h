// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Library/MediaViewerLibraryEntry.h"

#include "AssetRegistry/AssetData.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "UObject/ObjectPtr.h"

#include "MediaViewerLibraryItem.generated.h"

class UTexture;
struct FSlateBrush;
struct FSlateColor;

namespace UE::MediaViewer
{
	class FMediaImageViewer;
}

/**
 * Contains all data for a Library item.
*/
USTRUCT()
struct FMediaViewerLibraryItem : public FMediaViewerLibraryEntry
{
	friend struct FMediaViewerLibraryGroup;
	friend struct FMediaViewerLibraryDynamicGroup;

public:
	GENERATED_BODY()

	template<class InObjectType>
	static InObjectType* LoadAssetFromString(const FString& InString)
	{
		if (InString.IsEmpty())
		{
			return nullptr;
		}

		return LoadObject<InObjectType>(GetTransientPackage(), *InString);
	}

	FMediaViewerLibraryItem();

	MEDIAVIEWER_API FMediaViewerLibraryItem(const FText& InName, const FText& InToolTip, bool bInTransient, 
		const FString& InStringValue = TEXT(""));

	MEDIAVIEWER_API FMediaViewerLibraryItem(const FGuid& InId, const FText& InName, 
		const FText& InToolTip, bool bInTransient, const FString& InStringValue = TEXT(""));

	virtual ~FMediaViewerLibraryItem() override = default;

	MEDIAVIEWER_API virtual FName GetItemType() const;

	MEDIAVIEWER_API virtual FText GetItemTypeDisplayName() const;

	MEDIAVIEWER_API virtual FSlateColor GetItemTypeColor() const;

	MEDIAVIEWER_API bool IsTransient() const;

	virtual TSharedPtr<FSlateBrush> CreateThumbnail() { return nullptr; }

	virtual TSharedPtr<UE::MediaViewer::FMediaImageViewer> CreateImageViewer() const { return nullptr; }

	virtual TSharedPtr<FMediaViewerLibraryItem> Clone() const { return nullptr; }

	MEDIAVIEWER_API const FString& GetStringValue() const;

	virtual TOptional<FAssetData> AsAsset() const { return {}; }

	//~ Begin FMediaViewerLibraryEntry
	virtual UE::MediaViewer::EMediaViewerLibraryEntryType GetEntryType() const override { return UE::MediaViewer::EMediaViewerLibraryEntryType::Item; }
	//~ End FMediaViewerLibraryEntry

protected:
	MEDIAVIEWER_API static FSlateColor GetClassColor(UClass* InClass);

	/** If all this entry and all its children are transient. */
	bool bTransient;

	/** Used to serialize the object. */
	UPROPERTY()
	FString StringValue;
};
