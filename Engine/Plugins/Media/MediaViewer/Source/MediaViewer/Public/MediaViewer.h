// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Logging/LogMacros.h"
#include "UObject/NameTypes.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMediaViewer, Log, All);

namespace UE::MediaViewer
{

namespace ToolbarSections
{
	const FLazyName ToolbarLeft = TEXT("ToolbarLeft");
	const FLazyName ToolbarCenter = TEXT("ToolbarCenter");
	const FLazyName ToolbarRight = TEXT("ToolbarRight");
}

namespace StatusBarSections
{
	const FLazyName StatusBarLeft = TEXT("StatusBarLeft");
	const FLazyName StatusBarCenter = TEXT("StatusBarCenter");
	const FLazyName StatusBarRight = TEXT("StatusBarRight");
}

enum class EMediaImageViewerPosition : uint8
{
	First = 0,
	Second = 1,
	COUNT = 2
};

struct FMediaViewerArgs
{
	bool bShowSidebar = true;
	bool bShowToolbar = true;
	bool bShowImageViewerStatusBar = true;
	bool bAllowABComparison = true;

	bool operator==(const FMediaViewerArgs& InOther) const
	{
		return bShowToolbar == InOther.bShowToolbar
			&& bShowSidebar == InOther.bShowSidebar
			&& bShowImageViewerStatusBar == InOther.bShowImageViewerStatusBar
			&& bAllowABComparison == InOther.bAllowABComparison;
	}
};

} // UE::MediaViewer
