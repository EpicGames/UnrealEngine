// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Math/MathFwd.h"
#include "MediaViewer.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/StructOnScope.h"

class FUICommandList;
enum class EMediaImageMirrorPlane : uint8;
enum class EMediaImageViewerActivePosition : uint8;
enum EOrientation : int;
struct FMediaViewerSettings;

namespace UE::MediaViewer
{

class FMediaImageViewer;
class IMediaViewerLibrary;
class SMediaViewerTab;
	
struct FMediaImageViewerEventParams
{
	TSharedPtr<FMediaImageViewer> FromViewer;
	FName EventName;
};

struct FMediaViewerDelegates
{
	DECLARE_DELEGATE_RetVal(FVector2D, FGetLocation)
	DECLARE_DELEGATE_RetVal(EOrientation, FGetOrientation)
	DECLARE_DELEGATE_OneParam(FSetOrientation, EOrientation)
	DECLARE_DELEGATE_RetVal(EMediaImageViewerActivePosition, FGetActiveView)
	DECLARE_DELEGATE_RetVal(bool, FGetBool)
	DECLARE_DELEGATE_RetVal(float, FGetFloat)
	DECLARE_DELEGATE_OneParam(FSetFloat, float)
	DECLARE_DELEGATE_OneParam(FAddOffset, const FVector&)
	DECLARE_DELEGATE_OneParam(FAddRotation, const FRotator&)
	DECLARE_DELEGATE_OneParam(FMultiplyScale, float)
	DECLARE_DELEGATE_ThreeParams(FSetTransform, const FVector&, const FRotator&, float)
	DECLARE_DELEGATE_RetVal(FMediaViewerSettings*, FGetSettings)
	DECLARE_DELEGATE_RetVal(TSharedRef<IMediaViewerLibrary>, FGetLibrary)
	DECLARE_DELEGATE_RetVal(TSharedPtr<FUICommandList>, FGetCommandList)
	DECLARE_DELEGATE_TwoParams(FSetAllMirrorPlaneFlags, EMediaImageMirrorPlane, bool /* bInSet */)

	DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<FMediaImageViewer>, FGetImageViewer, EMediaImageViewerPosition)
	DECLARE_DELEGATE_TwoParams(FSetImageViewer, EMediaImageViewerPosition, const TSharedRef<FMediaImageViewer>&)
	DECLARE_DELEGATE_OneParam(FSimpleForPosition, EMediaImageViewerPosition)
	DECLARE_DELEGATE_RetVal_OneParam(FIntPoint, FGetLocationForPosition, EMediaImageViewerPosition)
	DECLARE_DELEGATE_RetVal_OneParam(bool, FGetBoolForPosition, EMediaImageViewerPosition)
	DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<FUICommandList>, FGetCommandListForPosition, EMediaImageViewerPosition)
	DECLARE_MULTICAST_DELEGATE_OneParam(FImageViewerEvent, const FMediaImageViewerEventParams&)

	//////////////////////////////
	// General viewer delegates //
	//////////////////////////////

	/** Changes to single view mode. */
	FSimpleDelegate SetSingleView;

	/** Changes to AB view. */
	FSimpleDelegate SetABView;
	
	/** Gets the orientation in A/B mode. Will return the currently selected orientation even if not in A/B mode. */
	FGetOrientation GetABOrientation;

	/** Sets the orientation in A/B mode. Will set the next expected orientation even if not in A/B mode. */
	FSetOrientation SetABOrientation;

	/** Returns a bitflag value indicating which image viewers are currently active (have images, not null or null image viewer). */
	FGetActiveView GetActiveView;

	/* Gets a pointer to the Media Viewer configurable settings. */
	FGetSettings GetSettings;
	
	/** Gets the size of the viewer paint area. */
	FGetLocation GetViewerSize;

	/** Gets the position of the viewer paint area in the window. */
	FGetLocation GetViewerPosition;

	/** Gets the local cursor location on the window. */
	FGetLocation GetCursorLocation;

	/** Swaps the A and B viewers, including their transforms. */
	FSimpleDelegate SwapAB;

	/** Returns true if the transforms of multi-image-viewer views are locked in sync. */
	FGetBool AreTransformsLocked;

	/** Toggles the transform lock for all image viewers. */
	FSimpleDelegate ToggleLockedTransform;

	/** Add an offset to all image viewers. */
	FAddOffset AddOffsetToAll;

	/** Adds a rotation to all image viewers. */
	FAddRotation AddRotationToAll;

	/** Multiplies the scale of all image viewers. */
	FMultiplyScale MultiplyScaleToAll;

	/** Multiplies the scale of all image viewers. */
	FMultiplyScale MultiplyScaleAroundCursorToAll;

	/** Sets the transform of all image viewers. */
	FSetTransform SetTransformToAll;

	/** Resets the transform of all image viewers to 0,0,0; 0,0,0; 1. */
	FSimpleDelegate ResetTransformToAll;

	/** Set the mirror plane flag (on/off) to all image viewers. */
	FSetAllMirrorPlaneFlags MirrorAll;

	/** Gets the opacity of the second/B image viewer. */
	FGetFloat GetSecondImageViewerOpacity;

	/** Sets the opacity of the second/B image viewer. */
	FSetFloat SetSecondImageViewerOpacity;

	/** Gets the splitter location in percent (0-100) */
	FGetFloat GetABSplitterLocation;

	/** Sets the splitter location in percent (0-100) */
	FSetFloat SetABSplitterLocation;

	/** Gets the image viewer library. */
	FGetLibrary GetLibrary;

	/* Forces a refresh of the view on next tick. */
	FSimpleDelegate RefreshView;

	/** Returns the command list for the general view. */
	FGetCommandList GetCommandList;

	/** Returns true if the mouse is over the viewer. */
	FGetBool IsOverViewer;

	//////////////////////////
	// Per-viewer delegates //
	//////////////////////////

	/** Gets the image viewer in the given position. */
	FGetImageViewer GetImageViewer;

	/** Sets the image viewer in the given position. */
	FSetImageViewer SetImageViewer;

	/** Sets the image viewer in the given position to the null viewer. */
	FSimpleForPosition ClearImageViewer;

	/** Gets the pixel coordinates hovered by the mouse for the image viewer in the given position. */
	FGetLocationForPosition GetPixelCoordinates;

	/** Returns true if the mouse is over the image viewer in the given position. */
	FGetBoolForPosition IsOverImage;

	/** Copies the transform of the image viewer in the given position to all other image viewers. */
	FSimpleForPosition CopyTransformToAll;

	/** Gets the command list for image viewer in the given position. */
	FGetCommandListForPosition GetCommandListForPosition;

	/** Generic event that can be broadcast between image viewers and controls. */
	FImageViewerEvent ImageViewerEvent;
};

} // UE::MediaViewer
