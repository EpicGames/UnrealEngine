// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Library/IMediaViewerLibrary.h"
#include "Templates/SharedPointerFwd.h"

class SWidget;
struct FGuid;

namespace UE::MediaViewer
{

/**
 * Displays a visual representation of a @see IMediaViewerLibrary.
 */
class IMediaViewerLibraryWidget
{
public:
	/**
	 * Delegate that gets called when an item is selected in the Library.
	 * The given @see FGuid identifies the item that was selected.
	 */
	DECLARE_DELEGATE_OneParam(FOnImageViewerOpened, const FGuid&)

	/**
	 * Delegate that gets called for creating a context menu for a group.
	 * Return @see SNullWidget::NullWidget to not show a context menu.
	 */
	DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<SWidget>, FOnGetGroupContextMenu, FName)

	/**
	 * Delegate that gets called for creating a context menu for a set of selected items.
	 * Return @see SNullWidget::NullWidget to not show a context menu.
	 */
	DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<SWidget>, FOnGetItemContextMenu, const TArray<FMediaViewerLibraryItem>&)

	/**
	 * Returns true to display the group. Return false to hide the group.
	 */
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FGroupFilter, const TSharedRef<const IMediaViewerLibrary>& InLibrary,
		const TSharedRef<const FMediaViewerLibraryGroup>& InGroup)

	struct FArgs
	{
		FOnImageViewerOpened OnImageViewerOpened;
		FOnGetGroupContextMenu OnGetGroupContextMenu;
		FOnGetItemContextMenu OnGetItemContextMenu;
		FGroupFilter GroupFilter;
	};

	/**
	 * Converts this interface to its underlying widget.
	 */
	virtual TSharedRef<SWidget> ToWidget() = 0;

	/**
	 * Returns the underlying IMediaViewerLibrary implementation.
	 */
	virtual TSharedRef<IMediaViewerLibrary> GetLibrary() const = 0;
};

} // UE::MediaViewer
