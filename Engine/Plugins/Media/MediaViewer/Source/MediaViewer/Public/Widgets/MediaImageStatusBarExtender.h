// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/NameTypes.h"

class FUICommandList;
class SHorizontalBox;

namespace UE::MediaViewer
{

struct FMediaImageStatusBarExtension
{
	/**
	 * Delegate that gets called for each extension when the status bar is constructed.
	 * The given @see SHorizontalBox is the layout container that the widget for the extension needs to be added to by calling.
	 * @see SHorizontalBox::AddSlot().
	 */
	DECLARE_DELEGATE_OneParam(FDelegate, const TSharedRef<SHorizontalBox>&);

	FName Hook;
	EExtensionHook::Position HookPosition = EExtensionHook::Before;
	TSharedPtr<FUICommandList> CommandList;
	FDelegate Delegate;
};

class FMediaImageStatusBarExtender
{
public:
	/**
	 * Adds an extension to the status bar.
	 * @param ExtensionHook The section the extension is applied to.
	 * @param HookPosition Where in relation to the section the extension is applied to.
	 * @param Commands The UI command list responsible for handling actions used in the widgets added in the extension.
	 * @param Delegate Delegate called for adding the extension widgets when the status bar is constructed.
	 */
	MEDIAVIEWER_API void AddExtension(FName InExtensionHook, EExtensionHook::Position InHookPosition,
		const TSharedPtr<FUICommandList>& InCommands, const FMediaImageStatusBarExtension::FDelegate& InDelegate);

	/** Used by the viewport to add extensions to the status bar. */
	void Apply(FName InExtensionHook, EExtensionHook::Position InHookPosition, const TSharedRef<SHorizontalBox>& InHorizontalBox) const;

private:
	/** List of extensions that get applied to the viewport status bar. */
	TArray<FMediaImageStatusBarExtension> Extensions;
};

} // UE::MediaViewer
