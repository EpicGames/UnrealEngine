// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "MediaViewer.h"

namespace UE::MediaViewer
{

class FMediaImageViewer;
class IMediaViewerLibrary;
enum class EMediaImageViewerPosition : uint8;

namespace Private
{
	class SMediaViewer;
}

class SMediaViewerTab : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SMediaViewerTab, SCompoundWidget)

	SLATE_BEGIN_ARGS(SMediaViewerTab)
		{}
		SLATE_ARGUMENT(FMediaViewerArgs, MediaViewerArgs)
		SLATE_ARGUMENT(TSharedPtr<FMediaImageViewer>, ImageViewerLeft)
		SLATE_ARGUMENT(TSharedPtr<FMediaImageViewer>, ImageViewerRight)
	SLATE_END_ARGS()

public:
	MEDIAVIEWER_API SMediaViewerTab();

	virtual ~SMediaViewerTab() override;

	MEDIAVIEWER_API void Construct(const FArguments& InArgs, const FMediaViewerArgs& InMediaViewerArgs);

	MEDIAVIEWER_API const FMediaViewerArgs& GetArgs() const;

	MEDIAVIEWER_API TSharedRef<IMediaViewerLibrary> GetLibrary() const;

	MEDIAVIEWER_API TSharedPtr<FMediaImageViewer> GetImageViewer(EMediaImageViewerPosition InPosition) const;

	MEDIAVIEWER_API void SetImageViewer(EMediaImageViewerPosition InPosition, TSharedPtr<FMediaImageViewer> InImageViewer);

	TSharedRef<Private::SMediaViewer> GetViewer() const;

protected:
	TSharedPtr<Private::SMediaViewer> Viewer;
};

} // UE::MediaViewer
